{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Take Recorder Multi-User synchronization",
	"Description": "Enables opt-in multi-user synchronization for Take Recorder.",
	"Category": "Virtual Production",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsBetaVersion": false,
	"Installed": false,
	"Plugins":
	[
		{
			"Name": "ConcertSyncClient",
			"Enabled": true
		},
		{
			"Name": "ConcertSyncCore",
			"Enabled": true
		},
		{
			"Name": "ConcertMain",
			"Enabled": true
		},
		{
			"Name": "Takes",
			"Enabled": true
		}
	],
	"Modules" :
	[
		{
			"Name": "ConcertTakeRecorder",
			"Type": "UncookedOnly",
			"LoadingPhase": "PostEngineInit"
		}
	]
}