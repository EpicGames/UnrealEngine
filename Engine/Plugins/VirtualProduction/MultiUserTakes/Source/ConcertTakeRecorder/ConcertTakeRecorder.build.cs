// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class ConcertTakeRecorder : ModuleRules
{
	public ConcertTakeRecorder(ReadOnlyTargetRules Target) : base(Target)
	{
        PublicDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
			}
		);

        PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Concert",
				"ConcertSyncClient",
				"ConcertSyncCore",
				"ConcertTransport",
				"EditorStyle",
				"InputCore",
				"LevelSequence",
				"Projects",
				"Slate",
				"SlateCore",
				"TakesCore",
				"TakeRecorder",
				"UnrealEd",
				"Engine"
			}
        );
    }
}
