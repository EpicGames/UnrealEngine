// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertTakeRecorderClientSessionCustomization.h"
#include "ConcertMessages.h"

#include "ConcertTakeRecorderMessages.h"
#include "ConcertTakeRecorderManager.h"

#include "DetailLayoutBuilder.h"
#include "IConcertClient.h"
#include "Framework/Views/TableViewMetadata.h"
#include "IConcertSyncClient.h"
#include "IConcertSyncClientModule.h"

#include "Widgets/Views/SListView.h"
#include "Widgets/Input/SCheckBox.h"
#include "EditorFontGlyphs.h"
#include "DetailWidgetRow.h"


#define LOCTEXT_NAMESPACE "ConcertTakeRecorderClientSessionCustomization"

namespace TakeRecordDetailsUI
{
	static const FName DisplayNameColumnName(TEXT("UserDisplayName"));
	static const FName RecordOnClientColumnName(TEXT("RecordOnClient"));
	static const FName TransactSourcesColumnName(TEXT("TransactSources"));
}

class STakeRecordDetailsRow : public SMultiColumnTableRow<TSharedPtr<FConcertClientRecordSetting>>
{
	SLATE_BEGIN_ARGS(STakeRecordDetailsRow) {}
	SLATE_END_ARGS()

public:

	void Construct(const FArguments& InArgs, TWeakPtr<IConcertSyncClient> InSyncClient,
				   TWeakPtr<IConcertClientSession> InClientSession,
				   TSharedPtr<FConcertClientRecordSetting> InClientInfo, const TSharedRef<STableViewBase>& InOwnerTableView)
	{
		SyncClient = MoveTemp(InSyncClient);
		ClientSession = MoveTemp(InClientSession);
		ClientRecordSetting = MoveTemp(InClientInfo);
		SMultiColumnTableRow<TSharedPtr<FConcertClientRecordSetting>>::Construct(FSuperRowType::FArguments(), InOwnerTableView);

		SetToolTipText(MakeAttributeSP(this, &STakeRecordDetailsRow::GetRowToolTip));
	}

public:
	TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override
	{
		using ThisType = STakeRecordDetailsRow;
		if (ColumnName == TakeRecordDetailsUI::DisplayNameColumnName)
		{
			return SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(FMargin(4.0f, 0.0f, 0.0f, 0.0f))
				[
					SNew(STextBlock)
					.Font(this, &STakeRecordDetailsRow::GetAvatarFont)
					.ColorAndOpacity(this, &STakeRecordDetailsRow::GetAvatarColor)
					.Text(FEditorFontGlyphs::Square)
				]

				// The client display name.
				+SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				[
					SNew(SBorder)
					.BorderImage(FAppStyle::GetBrush("NoBorder"))
					.ColorAndOpacity(FLinearColor(0.75f, 0.75f, 0.75f))
					.Padding(FMargin(6.0f, 4.0f))
					[
						SNew(STextBlock)
						.Font(FAppStyle::GetFontStyle("BoldFont"))
						.Text(GetDisplayName())
					]
				];
		}
		else if (ColumnName == TakeRecordDetailsUI::RecordOnClientColumnName)
		{
			return  SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.HAlign(HAlign_Left)
				[
					SAssignNew(RecordOnClient,SCheckBox)
					.IsChecked(this, &ThisType::GetRecordOnClientState)
					.OnCheckStateChanged(this, &ThisType::SetRecordOnClientState)
					.IsEnabled(this, &ThisType::CheckBoxEnabled)
					.ToolTipText_Lambda([this](){return GetRecordOnClientToolTip();})
				];
		}
		else
		{
			check(ColumnName == TakeRecordDetailsUI::TransactSourcesColumnName);

			return  SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.HAlign(HAlign_Left)
				[
					SAssignNew(TransactSources,SCheckBox)
					.IsChecked(this, &ThisType::GetTransactSourceState)
					.OnCheckStateChanged(this, &ThisType::SetTransactSourceState)
					.IsEnabled(this, &ThisType::CheckBoxEnabled)
					.ToolTipText_Lambda([this](){return GetTransactSourceToolTip();})
				];
		}
	}

	FText GetToolTipNoSync()
	{
		return FText::Format(LOCTEXT("NoSyncEnabled_ToolTip", "Synchronize Take Recorder Transactions Disabled on {0}"), GetDisplayName());
	}

	FText GetRecordOnClientToolTip()
	{
		TSharedPtr<FConcertClientRecordSetting> RecordSettingPin = ClientRecordSetting.Pin();
		if(RecordSettingPin.IsValid())
		{
			if (RecordSettingPin->bTakeSyncEnabled)
			{
				return LOCTEXT("RecordOnClient_ToolTip", "Sets whether this client should receive take record events.");
			}
		}
		return GetToolTipNoSync();
	}

	FText GetTransactSourceToolTip()
	{
		TSharedPtr<FConcertClientRecordSetting> RecordSettingPin = ClientRecordSetting.Pin();
		if(RecordSettingPin.IsValid())
		{
			if (RecordSettingPin->bTakeSyncEnabled)
			{
				return LOCTEXT("TransactSources_ToolTip", "Sets whether this client should transmit any take sources to clients.");
			}
		}
		return GetToolTipNoSync();
	}

	ECheckBoxState GetRecordOnClientState() const
	{
		TSharedPtr<FConcertClientRecordSetting> RecordSettingPin = ClientRecordSetting.Pin();
		if(RecordSettingPin.IsValid())
		{
			if (RecordSettingPin->Settings.bRecordOnClient)
			{
				return ECheckBoxState::Checked;
			}
		}
		return ECheckBoxState::Unchecked;
	}
	ECheckBoxState GetTransactSourceState() const
	{
		TSharedPtr<FConcertClientRecordSetting> RecordSettingPin = ClientRecordSetting.Pin();
		if(RecordSettingPin.IsValid())
		{
			if (RecordSettingPin->Settings.bTransactSources)
			{
				return ECheckBoxState::Checked;
			}
		}
		return ECheckBoxState::Unchecked;
	}

	void SetRecordOnClientState(ECheckBoxState InValue)
	{
		TSharedPtr<FConcertClientRecordSetting> RecordSettingPin = ClientRecordSetting.Pin();
		if(RecordSettingPin.IsValid())
		{
			RecordSettingPin->Settings.bRecordOnClient = ECheckBoxState::Checked == InValue?true:false;
			NotifyChange(RecordSettingPin);
		}
	}

	void SetTransactSourceState(ECheckBoxState InValue)
	{
		TSharedPtr<FConcertClientRecordSetting> RecordSettingPin = ClientRecordSetting.Pin();
		if(RecordSettingPin.IsValid())
		{
			RecordSettingPin->Settings.bTransactSources = ECheckBoxState::Checked == InValue?true:false;
			NotifyChange(RecordSettingPin);
		}
	}

	bool CheckBoxEnabled() const
	{
		TSharedPtr<FConcertClientRecordSetting> Setting = ClientRecordSetting.Pin();
		if (Setting.IsValid())
		{
			return Setting->bTakeSyncEnabled;
		}
		return false;
	}

	void UpdateUI()
	{
		TSharedPtr<FConcertClientRecordSetting> Setting = ClientRecordSetting.Pin();
		if (Setting.IsValid())
		{
			if (RecordOnClient.IsValid())
			{
				RecordOnClient->SetEnabled(Setting->bTakeSyncEnabled);
				RecordOnClient->SetIsChecked(Setting->Settings.bRecordOnClient);
				RecordOnClient->SetToolTipText(GetRecordOnClientToolTip());
			}
			if (TransactSources.IsValid())
			{
				TransactSources->SetEnabled(Setting->bTakeSyncEnabled);
				TransactSources->SetIsChecked(Setting->Settings.bTransactSources);
				TransactSources->SetToolTipText(GetTransactSourceToolTip());
			}
		}
	}

	FText GetRowToolTip() const
	{
		// This is a tooltip for the entire row. Like display name, the tooltip will not update in real time if the user change its
		// settings. See GetDisplayName() for more info.
		TSharedPtr<FConcertClientRecordSetting> RecordSettingPin = ClientRecordSetting.Pin();
		return RecordSettingPin.IsValid() ? RecordSettingPin->Details.ToDisplayString() : FText();
	}

	FText GetDisplayName() const
	{
		TSharedPtr<FConcertClientRecordSetting> RecordSettingPin = ClientRecordSetting.Pin();
		if (RecordSettingPin.IsValid())
		{
			// NOTE: The display name doesn't update in real time at the moment because the concert setting are not propagated
			//       until the client disconnect/reconnect. Since those settings should not change often, this should not
			//       be a major deal breaker for the users.
			TSharedPtr<IConcertClientSession> ClientSessionPin = ClientSession.Pin();
			if (ClientSessionPin.IsValid() && RecordSettingPin->Details.ClientEndpointId == ClientSessionPin->GetSessionClientEndpointId())
			{
				return FText::Format(LOCTEXT("ClientDisplayNameIsYouFmt", "{0} (You)"),
									 FText::FromString(ClientSessionPin->GetLocalClientInfo().DisplayName));
			}

			// Return the ClientInfo cached.
			return FText::FromString(RecordSettingPin->Details.ClientInfo.DisplayName);
		}

		return FText();
	}

	FSlateFontInfo GetAvatarFont() const
	{
		static const FName ButtonIconSyle = TEXT("FontAwesome.10");
		// This font is used to render a small square box filled with the avatar color.
		FSlateFontInfo ClientIconFontInfo = FAppStyle::Get().GetFontStyle(ButtonIconSyle);
		ClientIconFontInfo.Size = 8;
		ClientIconFontInfo.OutlineSettings.OutlineSize = 1;

		TSharedPtr<FConcertClientRecordSetting> RecordSettingPin = ClientRecordSetting.Pin();
		if (RecordSettingPin.IsValid())
		{
			FConcertSessionClientInfo& Client = RecordSettingPin->Details;
			FLinearColor ClientOutlineColor = Client.ClientInfo.AvatarColor * 0.6f; // Make the font outline darker.
			ClientOutlineColor.A = Client.ClientInfo.AvatarColor.A; // Put back the original alpha.
			ClientIconFontInfo.OutlineSettings.OutlineColor = ClientOutlineColor;
		}
		else
		{
			ClientIconFontInfo.OutlineSettings.OutlineColor = FLinearColor(0.75, 0.75, 0.75); // This is an arbitrary color.
		}

		return ClientIconFontInfo;
	}

	FSlateColor GetAvatarColor() const
	{
		TSharedPtr<FConcertClientRecordSetting> RecordSettingPin = ClientRecordSetting.Pin();
		if (RecordSettingPin.IsValid())
		{
			return RecordSettingPin->Details.ClientInfo.AvatarColor;
		}

		return FSlateColor(FLinearColor(0.75, 0.75, 0.75)); // This is an arbitrary color.
	}

	void NotifyChange(TSharedPtr<FConcertClientRecordSetting>& RecordSettingPin)
	{
		check(RecordSettingPin.IsValid());
		const FConcertClientRecordSetting& Ref = *RecordSettingPin.Get();
		SettingChangeDelegate.Broadcast(Ref);
	}

	FOnConcertRecordSettingChanged& SettingChanged()
	{
		return SettingChangeDelegate;
	}

private:
	TSharedPtr<SCheckBox>		 RecordOnClient;
	TSharedPtr<SCheckBox>		 TransactSources;

	TWeakPtr<IConcertSyncClient> SyncClient;
	TWeakPtr<IConcertClientSession> ClientSession;
	TWeakPtr<FConcertClientRecordSetting> ClientRecordSetting;
	FOnConcertRecordSettingChanged		  SettingChangeDelegate;

};


FConcertClientRecordSetting  GetLocalSessionConcertClientAsRecordSetting(
	UConcertSessionRecordSettings const* RecordSettings,
	TSharedPtr<IConcertClientSession>& Session)
{
	UConcertTakeSynchronization const* TakeSync = GetDefault<UConcertTakeSynchronization>();
	const FConcertClientInfo &ClientInfo = Session->GetLocalClientInfo();
	FConcertClientRecordSetting LocalClientRecordSetting;
	LocalClientRecordSetting.Details.ClientEndpointId =
		Session->GetSessionClientEndpointId();
	LocalClientRecordSetting.Details.ClientInfo = ClientInfo;
	LocalClientRecordSetting.Settings = RecordSettings->LocalSettings;
	LocalClientRecordSetting.bTakeSyncEnabled = TakeSync->bSyncTakeRecordingTransactions;
	return MoveTemp(LocalClientRecordSetting);
}

void FConcertTakeRecorderClientSessionCustomization::PopulateClientList()
{
	Clients.Reset();
	TSharedPtr<IConcertSyncClient> ConcertSyncClient = IConcertSyncClientModule::Get().GetClient(TEXT("MultiUser"));
	IConcertClientRef ConcertClient = ConcertSyncClient->GetConcertClient();
	TSharedPtr<IConcertClientSession> Session = ConcertClient->GetCurrentSession();

	if (Session)
	{
		UConcertSessionRecordSettings const* RecordSettings = GetDefault<UConcertSessionRecordSettings>();

		Clients.Reserve(RecordSettings->RemoteSettings.Num()+1);

		FConcertClientRecordSetting Local = GetLocalSessionConcertClientAsRecordSetting(RecordSettings,Session);

		Clients.Emplace(MakeShared<FConcertClientRecordSetting>(MoveTemp(Local)));
		for (FConcertClientRecordSetting const& Client : RecordSettings->RemoteSettings)
		{
			Clients.Emplace(MakeShared<FConcertClientRecordSetting>(Client));
		}
	}

	if (ClientsListViewWeak.IsValid())
	{
		TSharedPtr<SListViewClientRecordSetting> ClientsListView = ClientsListViewWeak.Pin();
		ClientsListView->RequestListRefresh();
	}
}

const FConcertClientRecordSetting* FConcertTakeRecorderClientSessionCustomization::FindClientSettings(const FGuid& EndpointSettings) const
{
	const int32 Index = InternalFindClientSettingsIndex(EndpointSettings);
	return Clients.IsValidIndex(Index) ? Clients[Index].Get() : nullptr;
}

void FConcertTakeRecorderClientSessionCustomization::UpdateClientSettings(EConcertClientStatus Status, const FConcertClientRecordSetting& RecordSetting)
{
	const int32 Index = InternalFindClientSettingsIndex(RecordSetting.Details.ClientEndpointId);

	switch (Status)
	{
	case EConcertClientStatus::Connected:
		Clients.Emplace(MakeShared<FConcertClientRecordSetting>(RecordSetting));
		break;
	case EConcertClientStatus::Updated:
		check(Index != INDEX_NONE);
		Clients[Index]->Settings = RecordSetting.Settings;
		Clients[Index]->bTakeSyncEnabled = RecordSetting.bTakeSyncEnabled;
		break;
	case EConcertClientStatus::Disconnected:
		check(Index != INDEX_NONE);
		Clients.RemoveAt(Index);
		break;
	default: checkNoEntry();
	}

	if (ClientsListViewWeak.IsValid())
	{
		const TSharedPtr<SListViewClientRecordSetting> ClientsListView = ClientsListViewWeak.Pin();
		if (Status == EConcertClientStatus::Disconnected
			|| Status == EConcertClientStatus::Connected )
		{
			ClientsListView->RequestListRefresh();
		}
		else
		{
			const TSharedPtr<STakeRecordDetailsRow> Row = StaticCastSharedPtr<STakeRecordDetailsRow>(ClientsListView->WidgetFromItem(Clients[Index]));
			if (Row.IsValid())
			{
				Row->UpdateUI();
			}
		}
	}
}

void FConcertTakeRecorderClientSessionCustomization::RecordSettingChange(const FConcertClientRecordSetting &RecordSetting)
{
	OnRecordSettingDelegate.Broadcast(RecordSetting);
}

int32 FConcertTakeRecorderClientSessionCustomization::InternalFindClientSettingsIndex(const FGuid& ClientEndpointId) const
{
	return Clients.IndexOfByPredicate([&ClientEndpointId](const TSharedPtr<FConcertClientRecordSetting>& Setting)
		{
			return Setting->Details.ClientEndpointId == ClientEndpointId;
		});
}

void FConcertTakeRecorderClientSessionCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailLayout)
{
	IDetailCategoryBuilder& RecordSettings = DetailLayout.EditCategory(TEXT("Multi-user Client Record Settings"));

	TSharedPtr<IPropertyHandle> SettingsProperty = DetailLayout.GetProperty(
		GET_MEMBER_NAME_CHECKED(UConcertSessionRecordSettings, LocalSettings));
	SettingsProperty->MarkHiddenByCustomization();


	TSharedPtr<IPropertyHandle> RemoteSettingsProperty = DetailLayout.GetProperty(
		GET_MEMBER_NAME_CHECKED(UConcertSessionRecordSettings, RemoteSettings));
	RemoteSettingsProperty->MarkHiddenByCustomization();

	PopulateClientList();
	UE_LOGF(LogConcertTakeRecorder, Display, "Customization Event");
	{
		FDetailWidgetRow& Row = RecordSettings.AddCustomRow(LOCTEXT("MultiUserRecordSettings", "Multi-user Client Record Settings"));
		auto HandleGenerateRow = [this](TSharedPtr<FConcertClientRecordSetting> InClientInfo,
									const TSharedRef<STableViewBase>& OwnerTable) -> TSharedRef<ITableRow>
		{
			TSharedPtr<IConcertSyncClient> ConcertSyncClient = IConcertSyncClientModule::Get().GetClient(TEXT("MultiUser"));
			IConcertClientRef ConcertClient = ConcertSyncClient->GetConcertClient();
			TSharedPtr<IConcertClientSession> ConcertClientSession = ConcertClient->GetCurrentSession();
			TSharedRef<STakeRecordDetailsRow> Row = SNew(STakeRecordDetailsRow, ConcertSyncClient, ConcertClientSession, InClientInfo, OwnerTable);
			Row->SettingChanged().AddSP(this,&FConcertTakeRecorderClientSessionCustomization::RecordSettingChange);
			return Row;
		};
		Row.WholeRowContent()
		[
			SAssignNew(ClientsListViewWeak,SListView<TSharedPtr<FConcertClientRecordSetting>>)
			.SelectionMode(ESelectionMode::None)
			.ListItemsSource(&Clients)
			.OnGenerateRow_Lambda(HandleGenerateRow)
			.HeaderRow
			(
				SNew(SHeaderRow)
				+SHeaderRow::Column(TakeRecordDetailsUI::DisplayNameColumnName)
				.DefaultLabel(LOCTEXT("UserDisplayName", "Display Name"))
				+SHeaderRow::Column(TakeRecordDetailsUI::RecordOnClientColumnName)
				.DefaultLabel(LOCTEXT("RecordOnClient", "Record On Client"))
				+SHeaderRow::Column(TakeRecordDetailsUI::TransactSourcesColumnName)
				.DefaultLabel(LOCTEXT("TransactSources", "Transact Sources"))
			)
		];
	}
}

#undef LOCTEXT_NAMESPACE
