// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "IDetailCustomization.h"

enum class EConcertClientStatus : uint8;
struct FConcertClientRecordSetting;
template <typename ItemType> class SListView;

DECLARE_MULTICAST_DELEGATE_OneParam(FOnConcertRecordSettingChanged, const FConcertClientRecordSetting&);

class FConcertTakeRecorderClientSessionCustomization : public IDetailCustomization
{
	using SListViewClientRecordSetting = SListView<TSharedPtr<FConcertClientRecordSetting>>;
public:

	//~ Begin IDetailCustomization Interface
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailLayout) override;
	//~ End IDetailCustomization Interface

	/** Updates the internal list of client data. Called whenever an external change is made (e.g. from remote client). */
	void UpdateClientSettings(EConcertClientStatus Status, const FConcertClientRecordSetting& RecordSetting);
	/** Resets the Clients list. Creates an entry for every client in the session. */
	void PopulateClientList();

	/** @return The settings saved for the client or nullptr */
	const FConcertClientRecordSetting* FindClientSettings(const FGuid& EndpointSettings) const;

	FOnConcertRecordSettingChanged& OnRecordSettingChanged() { return OnRecordSettingDelegate; }

private:
	
	FOnConcertRecordSettingChanged OnRecordSettingDelegate;

	/** State of the local and remote clients. Updated externally by UpdateClientSettings. Remote clients trigger networked events when they change the settings. */
	TArray<TSharedPtr<FConcertClientRecordSetting>> Clients;
	TWeakPtr<SListViewClientRecordSetting>		    ClientsListViewWeak;

	/** Called when a setting is changed by the customization */
	void RecordSettingChange(const FConcertClientRecordSetting &RecordSetting);
	
	/** @return The settings saved for the client or nullptr */
	int32 InternalFindClientSettingsIndex(const FGuid& ClientEndpointId) const;
};
