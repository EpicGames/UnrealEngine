// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertTakeRecorderManager.h"

#include "Algo/Count.h"
#include "ConcertMessages.h"
#include "ConcertSequencerMessages.h"
#include "ConcertWorkspaceData.h"
#include "Engine/Engine.h"
#include "IConcertClientTransactionBridge.h"
#include "IConcertClientPackageBridge.h"
#include "IConcertClientSequencerManager.h"
#include "IConcertClient.h"
#include "IConcertSyncClient.h"
#include "IConcertSyncClientModule.h"
#include "ConcertSyncArchives.h"
#include "ConcertTakeRecorderStyle.h"

#include "MultiUserTakesFunctionLibrary.h"

#include "ITakeRecorderModule.h"
#include "Misc/Guid.h"
#include "Recorder/TakeRecorder.h"
#include "TakePreset.h"
#include "TakeRecorderSources.h"
#include "TakeRecorderSettings.h"
#include "TakeRecorderSources.h"
#include "Recorder/TakeRecorderPanel.h"
#include "Recorder/TakeRecorderBlueprintLibrary.h"
#include "TakeMetaData.h"
#include "LevelSequence.h"

#include "PackageTools.h"
#include "Misc/PackageName.h"

#include "UObject/Package.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/SOverlay.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"


#include "Misc/MessageDialog.h"

#include "ConcertTakeRecorderSynchronizationCustomization.h"
#include "ConcertTakeRecorderClientSessionCustomization.h"

DEFINE_LOG_CATEGORY(LogConcertTakeRecorder);

// Enable Take Syncing
static int32 bConcertEnableTakeRecorderSync = 1;
static FAutoConsoleVariableRef  CVarEnableTakeSync(TEXT("Concert.EnableTakeRecorderSync"), bConcertEnableTakeRecorderSync, TEXT("Enable Concert Take Recorder Syncing."));

static int32 bConcertUseTakePresetPathForRecord = 0;
static FAutoConsoleVariableRef  CVarEnableTakePresetPathSync(TEXT("Concert.UseTakePresetPath"), bConcertUseTakePresetPathForRecord, TEXT("Use Take Presets for Take Recording."));

static int32 bEnableSkipHotReloadHint = 0;
static FAutoConsoleVariableRef  CVarEnableHotReloadHint(TEXT("Concert.TakeRecorderSkipHotReloadHint"), bEnableSkipHotReloadHint, TEXT("Should we indicate to the clients that hot reload can be skipped for assets generated via take recorder."));

static int32 bSkipPackageFilter = 0;
static FAutoConsoleVariableRef  CVarSkipPackageFilter(TEXT("Concert.TakeRecorderSkipPackageFilter"), bSkipPackageFilter, TEXT("Indicates if we should skip running the package filter for take recorder while in a Multi-user session. This will force all packages to be use default behavior and _temp packages may not get excluded."));

#define LOCTEXT_NAMESPACE "ConcertTakeRecorder"

ITakeRecorderModule& FTakeRecorderRecorderManagerGetModule()
{
	static const FName ModuleName = "TakeRecorder";
	return FModuleManager::LoadModuleChecked<ITakeRecorderModule>(ModuleName);
}

FConcertTakeRecorderManager::FConcertTakeRecorderManager()
{
	if (TSharedPtr<IConcertSyncClient> ConcertSyncClient = IConcertSyncClientModule::Get().GetClient(TEXT("MultiUser")))
	{
		IConcertClientRef ConcertClient = ConcertSyncClient->GetConcertClient();
		ConcertClient->OnSessionStartup().AddRaw(this, &FConcertTakeRecorderManager::Register);
		ConcertClient->OnSessionShutdown().AddRaw(this, &FConcertTakeRecorderManager::Unregister);

		if (TSharedPtr<IConcertClientSession> ConcertClientSession = ConcertClient->GetCurrentSession())
		{
			Register(ConcertClientSession.ToSharedRef());
		}

		ConcertClient->OnSessionConnectionChanged().AddRaw(this, &FConcertTakeRecorderManager::OnSessionConnectionChanged);
		RegisterExtensions();

		UTakeRecorder::OnRecordingInitialized().AddRaw(this, &FConcertTakeRecorderManager::OnTakeRecorderInitialized);
		FTakeRecorderParameterDelegate ParamHandler = FTakeRecorderParameterDelegate::CreateRaw(this, &FConcertTakeRecorderManager::SetupTakeParametersForMultiuser);
		UTakeRecorder::TakeInitializeParameterOverride().RegisterHandler(TEXT("ConcertTakeRecord"), MoveTemp(ParamHandler));
	}
	else
	{
		ensureAlwaysMsgf(false, TEXT("Could not find MultiUser client when creating ConcertTakeRecorderManager."));
	}
}

FConcertTakeRecorderManager::~FConcertTakeRecorderManager()
{
	UTakeRecorder::OnRecordingInitialized().RemoveAll(this);
	UTakeRecorder::TakeInitializeParameterOverride().UnregisterHandler(TEXT("ConcertTakeRecord"));

	if (TSharedPtr<IConcertSyncClient> ConcertSyncClient = IConcertSyncClientModule::Get().GetClient(TEXT("MultiUser")))
	{
		IConcertClientRef ConcertClient = ConcertSyncClient->GetConcertClient();
		ConcertClient->OnSessionStartup().RemoveAll(this);
		ConcertClient->OnSessionShutdown().RemoveAll(this);
	}

	UnregisterExtensions();
}

namespace {

void RegisterObjectCustomizations()
{
	ITakeRecorderModule& Module = FTakeRecorderRecorderManagerGetModule();

	UConcertTakeSynchronization* TakeSync = GetMutableDefault<UConcertTakeSynchronization>();
	Module.RegisterExternalObject(TakeSync);

	UConcertSessionRecordSettings* RecordSettings = GetMutableDefault<UConcertSessionRecordSettings>();
	Module.RegisterExternalObject(RecordSettings);
}

void UnregisterObjectCustomizations()
{
	ITakeRecorderModule& Module = FTakeRecorderRecorderManagerGetModule();

	UConcertSessionRecordSettings* Settings = GetMutableDefault<UConcertSessionRecordSettings>();
	Module.UnregisterExternalObject(Settings);

	UConcertTakeSynchronization * TakeSync = GetMutableDefault<UConcertTakeSynchronization>();
	Module.UnregisterExternalObject(TakeSync);
}
}

void FConcertTakeRecorderManager::Register(TSharedRef<IConcertClientSession> InSession)
{
	UE_LOGF(LogConcertTakeRecorder, Display, "Multi-user Take Recorder Session Startup: %ls", *InSession->GetSessionInfo().SessionName);
	WeakSession = nullptr;

	if (bConcertUseTakePresetPathForRecord == 0)
	{
		// We need to start in a consistent state for multi-user take recording.  So we clear the existing level sequence
		// on session start.
		//
		ITakeRecorderModule& TakeRecorderModule = FModuleManager::LoadModuleChecked<ITakeRecorderModule>("TakeRecorder");
		Preset = UTakePreset::AllocateTransientPreset(nullptr);
		check(Preset);
		Preset->CreateLevelSequence();
		ULevelSequence *LevelSequence = Preset->GetLevelSequence();
		check(LevelSequence);
		LevelSequence->FindOrAddMetaData<UTakeRecorderSources>();
	}


	// Hold onto the session so we can trigger events
	WeakSession = InSession;

	// Register our events
	InSession->RegisterCustomEventHandler<FConcertTakeInitializedEvent>(this, &FConcertTakeRecorderManager::OnTakeInitializedEvent);
	InSession->RegisterCustomEventHandler<FConcertRecordingFinishedEvent>(this, &FConcertTakeRecorderManager::OnRecordingFinishedEvent);
	InSession->RegisterCustomEventHandler<FConcertRecordingCancelledEvent>(this, &FConcertTakeRecorderManager::OnRecordingCancelledEvent);

	InSession->RegisterCustomEventHandler<FConcertRecordSettingsChangeEvent>(this, &FConcertTakeRecorderManager::OnRecordSettingsChangeEvent);
	InSession->RegisterCustomEventHandler<FConcertMultiUserSyncChangeEvent>(this, &FConcertTakeRecorderManager::OnMultiUserSyncChangeEvent);
	InSession->RegisterCustomEventHandler<FConcertRecordingNamedLevelSequenceEvent>(this, &FConcertTakeRecorderManager::OnNamedLevelSequenceEvent);
	if (InSession->GetConnectionStatus() == EConcertConnectionStatus::Connected)
	{
		OnSessionConnectionChanged(*InSession, EConcertConnectionStatus::Connected);
	}

	if(bConcertEnableTakeRecorderSync > 0)
	{
		RegisterObjectCustomizations();
	}
}

void FConcertTakeRecorderManager::Unregister(TSharedRef<IConcertClientSession> InSession)
{
	UE_LOGF(LogConcertTakeRecorder, Display, "Multi-user Take Recorder Session Shutdown: %ls", *InSession->GetSessionInfo().SessionName);

	if (TSharedPtr<IConcertClientSession> Session = WeakSession.Pin())
	{
		check(Session == InSession);

		UnregisterObjectCustomizations();
		Session->UnregisterCustomEventHandler<FConcertRecordingCancelledEvent>(this);
		Session->UnregisterCustomEventHandler<FConcertRecordingFinishedEvent>(this);
		Session->UnregisterCustomEventHandler<FConcertTakeInitializedEvent>(this);

		Session->UnregisterCustomEventHandler<FConcertMultiUserSyncChangeEvent>(this);
		Session->UnregisterCustomEventHandler<FConcertRecordSettingsChangeEvent>(this);

		Session->UnregisterCustomEventHandler<FConcertRecordingNamedLevelSequenceEvent>(this);
	}
	Preset = nullptr;
	WeakSession.Reset();
}

const FConcertClientRecordSetting* FConcertTakeRecorderManager::FindClientRecorderSetting(const FGuid& EndpointId) const
{
	return Customization ? Customization->FindClientSettings(EndpointId) : nullptr;
}

bool FConcertTakeRecorderManager::EditClientSettings(
	const FGuid& EndpointId,
	TFunctionRef<void(FTakeRecordSettings& Settings)> ModifierFunc,
	TOptional<TFunctionRef<bool(const FTakeRecordSettings& Settings)>> Predicate
	)
{
	const FConcertClientRecordSetting* Setting = FindClientRecorderSetting(EndpointId);
	if (!Setting)
	{
		return false;
	}

	const bool bFailedPredicate = Predicate && !(*Predicate)(Setting->Settings);
	if (bFailedPredicate)
	{
		return true;
	}

	// This unnecessarily copies a bunch of strings in FConcertSessionClientInfo as well but it keeps this logic simple...
	// Should refactor FConcertClientRecordSetting to contain a sub-struct that is then copied.
	FConcertClientRecordSetting SettingCopy = *Setting;
	ModifierFunc(SettingCopy.Settings);

	// Update the other clients...
	RecordSettingChange(SettingCopy);
	// ... and the local UI
	if (Customization)
	{
		Customization->UpdateClientSettings(EConcertClientStatus::Updated, SettingCopy);
	}

	return true;
}

void FConcertTakeRecorderManager::RegisterExtensions()
{
	ITakeRecorderModule& Module = FTakeRecorderRecorderManagerGetModule();

	Module.GetToolbarExtensionGenerators().AddRaw(this, &FConcertTakeRecorderManager::CreateExtensionWidget);
	Module.GetRecordButtonExtensionGenerators().AddRaw(this, &FConcertTakeRecorderManager::CreateRecordButtonOverlay);
	Module.GetRecordErrorCheckGenerator().AddRaw(this, &FConcertTakeRecorderManager::ReportRecordingError);
	Module.GetCanReviewLastRecordedLevelSequenceDelegate().BindRaw(this, &FConcertTakeRecorderManager::CanReviewLastRecordedSequence);

	if (GIsEditor)
	{
		if (TSharedPtr<IConcertSyncClient> ConcertSyncClient = IConcertSyncClientModule::Get().GetClient(TEXT("MultiUser")))
		{
			IConcertClientTransactionBridge* TransactionBridge = ConcertSyncClient->GetTransactionBridge();
			check(TransactionBridge != nullptr);

			TransactionBridge->RegisterTransactionFilter(TEXT("ConcertTakes"), FOnFilterTransactionDelegate::CreateRaw(this, &FConcertTakeRecorderManager::ShouldObjectBeTransacted));

			IConcertClientPackageBridge* PackageBridge = ConcertSyncClient->GetPackageBridge();
			check(PackageBridge);
			PackageBridge->RegisterPackageFilter(TEXT("ConcertTakes"), FPackageFilterDelegate::CreateRaw(this, &FConcertTakeRecorderManager::ShouldPackageBeFiltered));
			PackageBridge->RegisterPackageHotReloadHint(TEXT("ConcertTakes"), FPackageHotReloadHintDelegate::CreateRaw(this, &FConcertTakeRecorderManager::CanSkipHotReload));
		}

		FPropertyEditorModule& PropertyEditorModule = FModuleManager::Get().LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

		TakeSyncDelegate = FConcertTakeRecorderSynchronizationCustomization::OnSyncPropertyValueChanged().AddRaw(
			this,&FConcertTakeRecorderManager::OnTakeSyncPropertyChange);

		PropertyEditorModule.RegisterCustomClassLayout(
			UConcertTakeSynchronization::StaticClass()->GetFName(),
			FOnGetDetailCustomizationInstance::CreateLambda(&MakeShared<FConcertTakeRecorderSynchronizationCustomization>));

		PropertyEditorModule.RegisterCustomClassLayout(
			UConcertSessionRecordSettings::StaticClass()->GetFName(),
			FOnGetDetailCustomizationInstance::CreateLambda([this]()
			{
				Customization = MakeShared<FConcertTakeRecorderClientSessionCustomization>();
				Customization->OnRecordSettingChanged().AddRaw(this,&FConcertTakeRecorderManager::RecordSettingChange);
				return Customization->AsShared();
			}));
	}
}

void FConcertTakeRecorderManager::UnregisterExtensions()
{
	ITakeRecorderModule* TakeRecorder = FModuleManager::Get().GetModulePtr<ITakeRecorderModule>("TakeRecorder");
	if (TakeRecorder)
	{
		TakeRecorder->GetCanReviewLastRecordedLevelSequenceDelegate().Unbind();
		TakeRecorder->GetToolbarExtensionGenerators().RemoveAll(this);
		TakeRecorder->GetRecordButtonExtensionGenerators().RemoveAll(this);
		TakeRecorder->GetRecordErrorCheckGenerator().RemoveAll(this);
	}

	if (GIsEditor)
	{
		FPropertyEditorModule* PropertyEditorModule = FModuleManager::Get().GetModulePtr<FPropertyEditorModule>("PropertyEditor");
		if (PropertyEditorModule)
		{
			if (UObjectInitialized())
			{
				PropertyEditorModule->UnregisterCustomClassLayout(UConcertTakeSynchronization::StaticClass()->GetFName());
				PropertyEditorModule->UnregisterCustomClassLayout(UConcertSessionRecordSettings::StaticClass()->GetFName());
			}

			FConcertTakeRecorderSynchronizationCustomization::OnSyncPropertyValueChanged().Remove(TakeSyncDelegate);
		}

		if (TSharedPtr<IConcertSyncClient> ConcertSyncClient = IConcertSyncClientModule::Get().GetClient(TEXT("MultiUser")))
		{
			IConcertClientTransactionBridge* TransactionBridge = ConcertSyncClient->GetTransactionBridge();
			check(TransactionBridge != nullptr);
			TransactionBridge->UnregisterTransactionFilter(TEXT("ConcertTakes"));

			IConcertClientPackageBridge* PackageBridge = ConcertSyncClient->GetPackageBridge();
			check(PackageBridge != nullptr);
			PackageBridge->UnregisterPackageFilter(TEXT("ConcertTakes"));
			PackageBridge->UnregisterPackageHotReloadHint(TEXT("ConcertTakes"));
		}
	}
}

namespace ConcertTakeManagerHelper
{
EVisibility UIVisFromBool(bool bValue)
{
	return bValue ? EVisibility::HitTestInvisible : EVisibility::Collapsed;
}
}

bool FConcertTakeRecorderManager::ShouldIconBeVisible() const
{
	ITakeRecorderModule& TakeRecorderModule = FModuleManager::LoadModuleChecked<ITakeRecorderModule>("TakeRecorder");
	UTakePreset* TakePreset = TakeRecorderModule.GetPendingTake();
	UConcertTakeSynchronization const* TakeSync = GetDefault<UConcertTakeSynchronization>();

	if (WeakSession.IsValid() && TakePreset && !bIsRecording && TakeSync->bSyncTakeRecordingTransactions && IsTakeSyncEnabled())
	{
		bool bShouldSuppressBecauseOfRemoteOpen = false;
		if (TSharedPtr<IConcertSyncClient> ConcertSyncClient = IConcertSyncClientModule::Get().GetClient(TEXT("MultiUser")))
		{
			const SIZE_T Recorders = CanRecord() + RemoteRecorders();
			bShouldSuppressBecauseOfRemoteOpen = Recorders > 1 && ConcertSyncClient->GetSequencerManager()->IsSequencerRemoteOpenEnabled();
		}
		ULevelSequence* LevelSequence = TakePreset->GetLevelSequence();
		if (LevelSequence)
		{
			UTakeRecorderSources* Sources = LevelSequence->FindMetaData<UTakeRecorderSources>();
			return !bShouldSuppressBecauseOfRemoteOpen && Sources && Sources->GetSources().Num() > 0;
		}
	}

	return false;
}

EVisibility FConcertTakeRecorderManager::GetMultiUserIconVisibility() const
{
	return ConcertTakeManagerHelper::UIVisFromBool(ShouldIconBeVisible());
}

void FConcertTakeRecorderManager::CreateRecordButtonOverlay(TArray<TSharedRef<SWidget>>& OutExtensions)
{
	OutExtensions.Add(
		SNew(SBox)
		.Visibility_Raw(this, &FConcertTakeRecorderManager::GetMultiUserIconVisibility)
		[
			SNew(SImage)
			.Image(FConcertTakeRecorderStyle::Get()->GetBrush("Concert.TakeRecorder.SyncTakes.Tiny"))
		]
	);
}

void FConcertTakeRecorderManager::CreateExtensionWidget(TArray<TSharedRef<SWidget>>& OutExtensions)
{
	const int ButtonBoxSize = 28;

	OutExtensions.Add(
		SNew(SBox)
		.Visibility_Raw(this, &FConcertTakeRecorderManager::HandleTakeSyncButtonVisibility)
		.WidthOverride(ButtonBoxSize)
		[
			SNew(SOverlay)
			+ SOverlay::Slot()
			[
 				SNew(SCheckBox)
 				.Padding(4.f)
 				.ToolTipText(LOCTEXT("ToggleTakeRecorderSyncTooltip", "Toggle Multi-User Take Recorder Sync. If the option is enabled, starting/stopping/canceling a take will be synchronized with other Multi-User clients."))
 				.Style(FAppStyle::Get(), "ToggleButtonCheckbox")
 				.ForegroundColor(FSlateColor::UseForeground())
				.IsChecked_Raw(this, &FConcertTakeRecorderManager::IsTakeSyncChecked)
				.OnCheckStateChanged_Raw(this, &FConcertTakeRecorderManager::HandleTakeSyncCheckBox)
 				[
					SNew(SImage)
					.Image(FConcertTakeRecorderStyle::Get()->GetBrush("Concert.TakeRecorder.SyncTakes.Small"))
 				]
			]
		]
	);

	if (IsTakeSyncEnabled())
	{
		RegisterObjectCustomizations();
	}
}


void FConcertTakeRecorderManager::OnTakeRecorderStarted(UTakeRecorder* TakeRecorder)
{
#if WITH_EDITOR
	if (GEngine)
	{
		GEngine->CancelTransaction(0);
	}
#endif
}

void FConcertTakeRecorderManager::OnTakeRecorderStopped(UTakeRecorder* TakeRecorder)
{
#if WITH_EDITOR
	if (GEngine && GEngine->CanTransact() && WeakSession.IsValid())
	{
		UE_LOGF(LogConcertTakeRecorder, Warning, "A transaction is pending but we are currently stopping a Take Recorder. Transactions during a Recorder Stop can cause asset locking between Multi-user clients.");
	}
#endif
}

void FConcertTakeRecorderManager::OnTakeRecorderInitialized(UTakeRecorder* TakeRecorder)
{
	UTakeRecorderPanel* Panel = UTakeRecorderBlueprintLibrary::GetTakeRecorderPanel();
	if (!Panel)
	{
		return;
	}

	TakeRecorder->OnRecordingFinished().AddRaw(this, &FConcertTakeRecorderManager::OnRecordingFinished);
	TakeRecorder->OnRecordingCancelled().AddRaw(this, &FConcertTakeRecorderManager::OnRecordingCancelled);
	TakeRecorder->OnRecordingStarted().AddRaw(this, &FConcertTakeRecorderManager::OnTakeRecorderStarted);
	TakeRecorder->OnRecordingStopped().AddRaw(this, &FConcertTakeRecorderManager::OnTakeRecorderStopped);

	TakeRecorder->OnStartPlayFrameModified().AddRaw(this, &FConcertTakeRecorderManager::OnFrameAdjustment);

	if (IsTakeSyncEnabled() && !bIsRecording && TakeRecorderState.LastStartedTake != TakeRecorder->GetName())
	{
		if (TSharedPtr<IConcertClientSession> Session = WeakSession.Pin())
		{
			if (!CanAnyRecord())
			{
				// If no-one can record then force the local client to have recording so that we can continue.
				UMultiUserTakesFunctionLibrary::SetRecordOnClientLocal(true);
			}

			LastLevelSequence = nullptr;

			ITakeRecorderModule& TakeRecorderModule = FModuleManager::LoadModuleChecked<ITakeRecorderModule>("TakeRecorder");
			UTakeMetaData* TakeMetaData = Panel->GetTakeMetaData();
			UTakePreset* TakePreset = TakeRecorderModule.GetPendingTake();

			if (TakeMetaData && TakePreset)
			{
				if (bConcertUseTakePresetPathForRecord > 0 && TakePreset->GetOutermost()->IsDirty())
				{
					TakeRecorderModule.OnForceSaveAsPreset().Execute();
					if (TakePreset->GetOutermost()->IsDirty())
					{
						FText WarningMessage(LOCTEXT("Warning_RevertChanges", "Cannot start a synchronized take since there are changes to the take preset. Either revert your changes or save the preset to start a synchronized take."));
						FMessageDialog::Open(EAppMsgType::Ok, WarningMessage);
						return;
					}
				}

				if (!CanRecord())
				{
					TakeRecorder->SetDisableSaveTick(true);
				}
				else
				{
					if (TSharedPtr<IConcertSyncClient> ConcertSyncClient = IConcertSyncClientModule::Get().GetClient(TEXT("MultiUser")))
					{
						ConcertSyncClient->GetSequencerManager()->SuspendSequencerPacing();
					}
				}
				FConcertTakeInitializedEvent TakeInitializedEvent;
				TakeInitializedEvent.TakeName = TakeRecorder->GetName();
				TakeInitializedEvent.TakePresetPath = TakeMetaData->GetPresetOrigin()->GetPathName();
				TakeInitializedEvent.Settings = GetDefault<UTakeRecorderUserSettings>()->Settings;
				TakeInitializedEvent.Project = GetDefault<UTakeRecorderProjectSettings>()->Settings;
				TakeInitializedEvent.TakeMetaDataPath = TakeMetaData->GetPathName();

				FConcertLocalIdentifierTable InLocalIdentifierTable;
				FConcertSyncObjectWriter Writer(&InLocalIdentifierTable, TakeMetaData, TakeInitializedEvent.TakeData, true, false);
				Writer.SerializeObject(TakeMetaData);

				InLocalIdentifierTable.GetState(TakeInitializedEvent.IdentifierState);
				UE_LOGF(LogConcertTakeRecorder, Display, "Sending custom event to initialize recording.");
				Session->SendCustomEvent(TakeInitializedEvent, Session->GetSessionClientEndpointIds(), EConcertMessageFlags::ReliableOrdered | EConcertMessageFlags::UniqueId);
				bIsRecording = true;
			}
		}
	}
}

void FConcertTakeRecorderManager::OnRecordingFinished(UTakeRecorder* TakeRecorder)
{
	if (IsTakeSyncEnabled() && TakeRecorderState.LastStoppedTake != TakeRecorder->GetName())
	{
		if (TSharedPtr<IConcertClientSession> Session = WeakSession.Pin())
		{
			FConcertRecordingFinishedEvent RecordingFinishedEvent;
			RecordingFinishedEvent.TakeName = TakeRecorder->GetName();
			Session->SendCustomEvent(RecordingFinishedEvent, Session->GetSessionClientEndpointIds(), EConcertMessageFlags::ReliableOrdered | EConcertMessageFlags::UniqueId);

			if (CanRecord())
			{
				if (TSharedPtr<IConcertSyncClient> ConcertSyncClient = IConcertSyncClientModule::Get().GetClient(TEXT("MultiUser")))
				{
					ConcertSyncClient->GetSequencerManager()->ResumeSequencerPacing();
				}

				LastLevelSequence = TakeRecorder->GetSequence();
				check(LastLevelSequence);
				FConcertRecordingNamedLevelSequenceEvent NamedSequence{LastLevelSequence->GetPathName()};
				Session->SendCustomEvent(NamedSequence, Session->GetSessionClientEndpointIds(), EConcertMessageFlags::ReliableOrdered);
			}
		}
	}

	TakeRecorder->OnRecordingFinished().RemoveAll(this);
	TakeRecorder->OnRecordingCancelled().RemoveAll(this);
	bIsRecording = false;
}

void FConcertTakeRecorderManager::OnRecordingCancelled(UTakeRecorder* TakeRecorder)
{
	if (IsTakeSyncEnabled() && TakeRecorderState.LastStoppedTake != TakeRecorder->GetName())
	{
		if (TSharedPtr<IConcertClientSession> Session = WeakSession.Pin())
		{
			FConcertRecordingCancelledEvent RecordingCancelledEvent;
			RecordingCancelledEvent.TakeName = TakeRecorder->GetName();
			Session->SendCustomEvent(RecordingCancelledEvent, Session->GetSessionClientEndpointIds(), EConcertMessageFlags::ReliableOrdered);
		}
	}

	TakeRecorder->OnRecordingFinished().RemoveAll(this);
	TakeRecorder->OnRecordingCancelled().RemoveAll(this);
	TakeRecorder->OnRecordingStarted().RemoveAll(this);
	TakeRecorder->OnRecordingFinished().RemoveAll(this);
	TakeRecorder->OnStartPlayFrameModified().RemoveAll(this);
	bIsRecording = false;
}

void FConcertTakeRecorderManager::OnFrameAdjustment(UTakeRecorder* TakeRecorder, const FFrameNumber& InPlaybackStartFrame)
{
	if (TSharedPtr<IConcertClientSession> Session = WeakSession.Pin())
	{
		ULevelSequence* LevelSequence = TakeRecorder->GetSequence();
		check(LevelSequence);

		FConcertSequencerTimeAdjustmentEvent TimeEvent;
		TimeEvent.SequenceObjectPath = LevelSequence->GetPathName();
		TimeEvent.PlaybackStartFrame = InPlaybackStartFrame;
		Session->SendCustomEvent(TimeEvent, Session->GetSessionClientEndpointIds(), EConcertMessageFlags::ReliableOrdered);
	}
}

void FConcertTakeRecorderManager::OnTakeInitializedEvent(const FConcertSessionContext&, const FConcertTakeInitializedEvent& InEvent)
{
	TakeRecorderState.LastStoppedTake = "";
	TakeRecorderState.LastStartedTake = InEvent.TakeName;

	if (IsTakeSyncEnabled() && CanRecord())
	{
		ITakeRecorderModule& TakeRecorderModule = FModuleManager::LoadModuleChecked<ITakeRecorderModule>("TakeRecorder");
		UTakePreset* TakePreset = Preset;
		if (bConcertUseTakePresetPathForRecord > 0)
		{
			TakePreset = Cast<UTakePreset>(StaticLoadObject(UObject::StaticClass(), nullptr, *InEvent.TakePresetPath));
		}

		if (TakePreset && TakePreset->GetLevelSequence())
		{
			// Stop the active recorder if it's running.
			UTakeRecorder* ActiveTakeRecorder = UTakeRecorder::GetActiveRecorder();
			if (ActiveTakeRecorder && ActiveTakeRecorder->GetState() == ETakeRecorderState::Started)
			{
				ActiveTakeRecorder->Stop();
			}

			UTakeMetaData* TakeMetadata = NewObject<UTakeMetaData>(GetTransientPackage(), NAME_None, EObjectFlags::RF_Transient);

			FConcertLocalIdentifierTable Table(InEvent.IdentifierState);
			FConcertSyncWorldRemapper Remapper(InEvent.TakeMetaDataPath, TakeMetadata->GetPathName());
			FConcertSyncObjectReader Reader(&Table, MoveTemp(Remapper), nullptr, TakeMetadata, InEvent.TakeData);
			Reader.SerializeObject(TakeMetadata);

			ULevelSequence* LevelSequence = TakePreset->GetLevelSequence();
			UTakeRecorderSources* Sources = LevelSequence->FindMetaData<UTakeRecorderSources>();

			FTakeRecorderParameters DefaultParams;
			DefaultParams.User = InEvent.Settings;
			DefaultParams.Project = InEvent.Project;

			UTakeRecorder* NewRecorder = NewObject<UTakeRecorder>(GetTransientPackage(), NAME_None, RF_Transient);

			LastLevelSequence = nullptr;

			bIsRecording = NewRecorder->Initialize(LevelSequence, Sources, TakeMetadata, DefaultParams);
			if (bIsRecording)
			{
				UE_LOGF(LogConcertTakeRecorder, Display, "Take initialized for recording by multi-user event.");
			}
			else
			{
				UE_LOGF(LogConcertTakeRecorder, Error, "Failed to initialize for recording by multi-user event.");
			}

		}
	}
}

void FConcertTakeRecorderManager::OnRecordingFinishedEvent(const FConcertSessionContext&, const FConcertRecordingFinishedEvent& InEvent)
{
	if (IsTakeSyncEnabled())
	{
		TakeRecorderState.LastStoppedTake = InEvent.TakeName;

		if (UTakeRecorder* ActiveTakeRecorder = UTakeRecorder::GetActiveRecorder())
		{
			if (bIsRecording)
			{
				LastLevelSequence = ActiveTakeRecorder->GetSequence();
				check(LastLevelSequence);
				FConcertRecordingNamedLevelSequenceEvent NamedSequence{LastLevelSequence->GetPathName()};
				TSharedPtr<IConcertClientSession> Session = WeakSession.Pin();
				Session->SendCustomEvent(NamedSequence, Session->GetSessionClientEndpointIds(), EConcertMessageFlags::ReliableOrdered);
			}
			ActiveTakeRecorder->Stop();
			bIsRecording = false;
		}
	}
}

void FConcertTakeRecorderManager::OnRecordingCancelledEvent(const FConcertSessionContext&, const FConcertRecordingCancelledEvent& InEvent)
{
	if (IsTakeSyncEnabled())
	{
		TakeRecorderState.LastStoppedTake = InEvent.TakeName;
		if (UTakeRecorder* ActiveTakeRecorder = UTakeRecorder::GetActiveRecorder())
		{
			ActiveTakeRecorder->Stop();
			bIsRecording = false;
		}
	}
}

void FConcertTakeRecorderManager::OnRecordSettingsChangeEvent(const FConcertSessionContext&, const FConcertRecordSettingsChangeEvent& InEvent)
{
	TSharedPtr<IConcertClientSession> Session = WeakSession.Pin();
	if (Session.IsValid())
	{
		UConcertSessionRecordSettings* RecordSettings = GetMutableDefault<UConcertSessionRecordSettings>();

		if (InEvent.EndpointId == Session->GetSessionClientEndpointId())
		{
			RecordSettings->LocalSettings = InEvent.Settings;
			if (Customization.IsValid())
			{
				FConcertClientRecordSetting Item;
				Item.Details.ClientEndpointId = InEvent.EndpointId;
				Item.Settings = InEvent.Settings;
				Item.bTakeSyncEnabled = GetDefault<UConcertTakeSynchronization>()->bSyncTakeRecordingTransactions;
				Customization->UpdateClientSettings(EConcertClientStatus::Updated, Item);
			}
		}
		else
		{
			for(FConcertClientRecordSetting& Item : RecordSettings->RemoteSettings)
			{
				if (Item.Details.ClientEndpointId == InEvent.EndpointId)
				{
					Item.Settings = InEvent.Settings;
					if (Customization.IsValid())
					{
						Customization->UpdateClientSettings(EConcertClientStatus::Updated, Item);
					}
				}
			}
		}
	}
}

void FConcertTakeRecorderManager::OnTakeSyncPropertyChange(bool Value)
{
	TSharedPtr<IConcertClientSession> Session = WeakSession.Pin();
	if (Session.IsValid())
	{
		FConcertMultiUserSyncChangeEvent OutEvent;
		OutEvent.EndpointId = Session->GetSessionClientEndpointId();
		OutEvent.bSyncTakeRecordingTransactions = Value;

		Session->SendCustomEvent(OutEvent, Session->GetSessionClientEndpointIds(), EConcertMessageFlags::ReliableOrdered);

		if (Customization.IsValid())
		{
			UConcertSessionRecordSettings const* RecordSettings = GetDefault<UConcertSessionRecordSettings>();
			FConcertClientRecordSetting Item;
			Item.Details.ClientEndpointId = Session->GetSessionClientEndpointId();
			Item.Settings = RecordSettings->LocalSettings;
			Item.bTakeSyncEnabled = Value;
			Customization->UpdateClientSettings(EConcertClientStatus::Updated, Item);
		}
	}
	UConcertTakeSynchronization* Sync = GetMutableDefault<UConcertTakeSynchronization>();
	Sync->SaveConfig();
}

void FConcertTakeRecorderManager::OnMultiUserSyncChangeEvent(const FConcertSessionContext&, const FConcertMultiUserSyncChangeEvent& InEvent)
{
	TSharedPtr<IConcertClientSession> Session = WeakSession.Pin();
	if (Session.IsValid())
	{
		UConcertSessionRecordSettings* RecordSettings = GetMutableDefault<UConcertSessionRecordSettings>();
		for(FConcertClientRecordSetting& Item : RecordSettings->RemoteSettings)
		{
			if (Item.Details.ClientEndpointId == InEvent.EndpointId
				&& Item.bTakeSyncEnabled != InEvent.bSyncTakeRecordingTransactions)
			{
				Item.bTakeSyncEnabled = InEvent.bSyncTakeRecordingTransactions;
				if (Customization.IsValid())
				{
					Customization->UpdateClientSettings(EConcertClientStatus::Updated, Item);
				}
			}
		}
	}
}

void FConcertTakeRecorderManager::OnSessionClientChanged(IConcertClientSession& Session, EConcertClientStatus ClientStatus, const FConcertSessionClientInfo& InClientInfo)
{
	check(WeakSession.IsValid());

	UConcertSessionRecordSettings* RecordSettings = GetMutableDefault<UConcertSessionRecordSettings>();
	TArray<FConcertClientRecordSetting>& Clients = RecordSettings->RemoteSettings;

	int32 Index = Clients.IndexOfByPredicate([&InClientInfo](const FConcertClientRecordSetting& Setting) {
		return Setting.Details.ClientEndpointId == InClientInfo.ClientEndpointId;
	});

	switch(ClientStatus)
	{
	case EConcertClientStatus::Connected:
	{
		check(Index == INDEX_NONE);
		UE_LOGF(LogConcertTakeRecorder, Display, "Session Client Change: %ls Connected.", *InClientInfo.ClientInfo.DisplayName);
		FConcertClientRecordSetting RecordSetting;
		RecordSetting.Details = InClientInfo;
		Clients.Emplace(MoveTemp(RecordSetting));

		SendInitialState(Session);
	}
	break;
	case EConcertClientStatus::Disconnected:
		UE_LOGF(LogConcertTakeRecorder, Display, "Session Client Change: %ls Disconnected.", *InClientInfo.ClientInfo.DisplayName);
		break;
	case EConcertClientStatus::Updated:
		break;
	};

	if (Customization.IsValid())
	{
		Customization->UpdateClientSettings(ClientStatus, Index == INDEX_NONE ? Clients.Last() : Clients[Index] );
	}

	if (ClientStatus == EConcertClientStatus::Disconnected && Index != INDEX_NONE)
	{
		Clients.RemoveAt(Index);
	}
	RecordSettings->SaveConfig();
}

void FConcertTakeRecorderManager::DisconnectFromSession()
{
	check(WeakSession.IsValid());
	{
		TSharedPtr<IConcertClientSession> Session = WeakSession.Pin();
		Session->OnSessionClientChanged().Remove(ClientChangeDelegate);
	}
}

void FConcertTakeRecorderManager::ConnectToSession(IConcertClientSession& InSession)
{
	UE_LOGF(LogConcertTakeRecorder, Display, "Multi-user Take Recorder Connected to Session: %ls", *InSession.GetSessionInfo().SessionName);

	ClientChangeDelegate = InSession.OnSessionClientChanged().AddRaw(this, &FConcertTakeRecorderManager::OnSessionClientChanged);

	SendInitialState(InSession);
	UpdateSessionClientList();
}

void FConcertTakeRecorderManager::OnSessionConnectionChanged(IConcertClientSession& InSession, EConcertConnectionStatus ConnectionStatus)
{
	if (ConnectionStatus == EConcertConnectionStatus::Connected)
	{
		DisconnectFromSession();
		ConnectToSession(InSession);
	}
	else if (ConnectionStatus == EConcertConnectionStatus::Disconnecting)
	{
		UE_LOGF(LogConcertTakeRecorder, Display, "Multi-user Take Recorder Disconnecting from Session: %ls", *InSession.GetSessionInfo().SessionName);
		DisconnectFromSession();
	}

	if (UTakeRecorder* ActiveTakeRecorder = UTakeRecorder::GetActiveRecorder())
	{
		ETakeRecorderState ActiveState = ActiveTakeRecorder->GetState();
		if (ActiveState != ETakeRecorderState::Stopped && ActiveState != ETakeRecorderState::Cancelled)
		{
			ActiveTakeRecorder->Cancel();
		}
	}
}

void FConcertTakeRecorderManager::ReportRecordingError(FText &OutputError)
{
	const SIZE_T Recorders = CanRecord() + RemoteRecorders();
	if(WeakSession.IsValid() && IsTakeSyncEnabled() && Recorders > 1)
	{
		if (TSharedPtr<IConcertSyncClient> ConcertSyncClient = IConcertSyncClientModule::Get().GetClient(TEXT("MultiUser")))
		{
			const bool bIsRemoteOpenEnabled = ConcertSyncClient->GetSequencerManager()->IsSequencerRemoteOpenEnabled();
			if (bIsRemoteOpenEnabled)
			{
				OutputError = LOCTEXT("ErrorWidget_RemoteOpenEnabled", "Sequencer remote open is enabled. It is recommended you disable this when recording on multiple clients.");
			}
		}
	}
}

bool FConcertTakeRecorderManager::CanAnyRecord() const
{
	UConcertSessionRecordSettings const* RecordSettings = GetDefault<UConcertSessionRecordSettings>();
	bool bCanRecord = CanRecord();
	for( const FConcertClientRecordSetting&  Remote : RecordSettings->RemoteSettings )
	{
		bCanRecord = bCanRecord || Remote.Settings.bRecordOnClient;
	}
	return bCanRecord;
}

bool FConcertTakeRecorderManager::CanRecord() const
{
	UConcertSessionRecordSettings const* RecordSettings = GetDefault<UConcertSessionRecordSettings>();
	UConcertTakeSynchronization const* TakeSync = GetDefault<UConcertTakeSynchronization>();
	return TakeSync->bSyncTakeRecordingTransactions && RecordSettings->LocalSettings.bRecordOnClient;
}

bool FConcertTakeRecorderManager::IsTakeSyncEnabled() const
{
	return bConcertEnableTakeRecorderSync > 0;
}

ECheckBoxState FConcertTakeRecorderManager::IsTakeSyncChecked() const
{
	return IsTakeSyncEnabled() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void FConcertTakeRecorderManager::HandleTakeSyncCheckBox(ECheckBoxState State) const
{
	if (State == ECheckBoxState::Checked)
	{
		bConcertEnableTakeRecorderSync = 1;
		RegisterObjectCustomizations();
	}
	else
	{
		UnregisterObjectCustomizations();
		bConcertEnableTakeRecorderSync = 0;
	}
}

EVisibility FConcertTakeRecorderManager::HandleTakeSyncButtonVisibility() const
{
	return  WeakSession.IsValid() ? EVisibility::Visible : EVisibility::Collapsed;
}

void FConcertTakeRecorderManager::SendInitialState(IConcertClientSession& Session)
{
	UConcertSessionRecordSettings const* Record = GetDefault<UConcertSessionRecordSettings>();
	FConcertRecordSettingsChangeEvent OutEvent;
	OutEvent.Settings   = Record->LocalSettings;
	OutEvent.EndpointId = Session.GetSessionClientEndpointId();
	Session.SendCustomEvent(OutEvent, Session.GetSessionClientEndpointIds(), EConcertMessageFlags::ReliableOrdered);

	UConcertTakeSynchronization const* TakeSync = GetDefault<UConcertTakeSynchronization>();
	FConcertMultiUserSyncChangeEvent TakeSyncEvent;
	TakeSyncEvent.EndpointId = OutEvent.EndpointId;
	TakeSyncEvent.bSyncTakeRecordingTransactions = TakeSync->bSyncTakeRecordingTransactions;
	Session.SendCustomEvent(TakeSyncEvent, Session.GetSessionClientEndpointIds(), EConcertMessageFlags::ReliableOrdered);
}

void FConcertTakeRecorderManager::RecordSettingChange(const FConcertClientRecordSetting& RecordSetting)
{
	UE_LOGF(LogConcertTakeRecorder, Display, "Record Setting Changed");

	TSharedPtr<IConcertClientSession> Session = WeakSession.Pin();
	if (Session.IsValid())
	{
		UConcertSessionRecordSettings* RecordSettings = GetMutableDefault<UConcertSessionRecordSettings>();
		if (RecordSetting.Details.ClientEndpointId == Session->GetSessionClientEndpointId())
		{
			RecordSettings->LocalSettings = RecordSetting.Settings;
		}
		else
		{
			for(FConcertClientRecordSetting& Item : RecordSettings->RemoteSettings)
			{
				if (Item.Details.ClientEndpointId == RecordSetting.Details.ClientEndpointId)
				{
					Item.Settings = RecordSetting.Settings;
				}
			}
		}

		RecordSettings->SaveConfig();
		const SIZE_T Recorders = CanRecord() + RemoteRecorders();
		if (!bHaveWarned && Recorders > 1)
		{
			FText ErrorText = LOCTEXT("ToggleTakeRecorderWarnNameTooltip",
									  "Multiple machines are recording. To prevent name conflicts, the multi-user name will be appended to the take path.");
			FNotificationInfo Info(ErrorText);
			Info.ExpireDuration = 5.0f;
			FSlateNotificationManager::Get().AddNotification(Info)->SetCompletionState(SNotificationItem::CS_None);

			// Add a notification to disable remote open feature.
			if (TSharedPtr<IConcertSyncClient> ConcertSyncClient = IConcertSyncClientModule::Get().GetClient(TEXT("MultiUser")))
			{
				const bool bRemoteOpen = ConcertSyncClient->GetSequencerManager() && ConcertSyncClient->GetSequencerManager()->IsSequencerRemoteOpenEnabled();
				if (bRemoteOpen)
				{
					FText ShouldDisableErrorText = LOCTEXT("ShouldDisableSequencerRemoteOpen",
											  "When multiple machines are recording, it is recommended that you disable sequencer remote open.");
					FNotificationInfo SequencerOpenInfo(ShouldDisableErrorText);
					SequencerOpenInfo.ExpireDuration = 10.0f;
					SequencerOpenInfo.HyperlinkText = LOCTEXT("DisableRemoteOpen", "Disable sequencer remote open.");
					SequencerOpenInfo.Hyperlink = FSimpleDelegate::CreateLambda([this, ConcertSyncClient]() {
						if (ConcertSyncClient && ConcertSyncClient->GetSequencerManager())
						{
							ConcertSyncClient->GetSequencerManager()->SetSequencerRemoteOpen(false);
						}
					});

					FSlateNotificationManager::Get().AddNotification(SequencerOpenInfo);
				} 
			}

			bHaveWarned = true;
		}

		if (Recorders == 1)
		{
			bHaveWarned = false;
		}
		FConcertRecordSettingsChangeEvent OutEvent;
		OutEvent.Settings   = RecordSetting.Settings;
		OutEvent.EndpointId = RecordSetting.Details.ClientEndpointId;
		Session->SendCustomEvent(OutEvent, Session->GetSessionClientEndpointIds(), EConcertMessageFlags::ReliableOrdered);
	}
}

void FConcertTakeRecorderManager::AddRemoteClient(const FConcertSessionClientInfo& ClientInfo)
{
	UConcertSessionRecordSettings* RecordSettings = GetMutableDefault<UConcertSessionRecordSettings>();
	FConcertClientRecordSetting Setting;
	Setting.Details = ClientInfo;

	RecordSettings->RemoteSettings.Emplace(MoveTemp(Setting));
}

void FConcertTakeRecorderManager::UpdateSessionClientList()
{
	TSharedPtr<IConcertClientSession> Session = WeakSession.Pin();

	check(Session.IsValid());

	TArray<FConcertSessionClientInfo> ConnectedClients = Session->GetSessionClients();

	UConcertSessionRecordSettings* RecordSettings = GetMutableDefault<UConcertSessionRecordSettings>();
	RecordSettings->RemoteSettings.Reset();
	RecordSettings->RemoteSettings.Reserve(ConnectedClients.Num());

	for (FConcertSessionClientInfo& Client : ConnectedClients)
	{
		UE_LOGF(LogConcertTakeRecorder, Display, "Session Client Add->%ls", *Client.ToDisplayString().ToString());
		AddRemoteClient(Client);
	}

	if (Customization)
	{
		Customization->PopulateClientList();
	}
	RecordSettings->SaveConfig();
}

namespace UE::TakeRecorderManager::Private
{
	bool IsDuplicateRemoteName(const FString& LocalDisplayName)
	{
		UConcertSessionRecordSettings const* RecordSettings = GetDefault<UConcertSessionRecordSettings>();
		SIZE_T Num = Algo::CountIf(
			RecordSettings->RemoteSettings,
			[LocalDisplayName](const FConcertClientRecordSetting&  Remote)
			{
				if (Remote.Settings.bRecordOnClient
					&& Remote.Details.ClientInfo.DisplayName == LocalDisplayName)
				{
					return true;
				}
				return false;
			});
		return Num > 0;
	}
	const FString TempPostfix = TEXT("_temp");
}

EPackageFilterResult FConcertTakeRecorderManager::ShouldPackageBeFiltered(const FConcertPackageInfo& InPackageInfo)
{
	bool bShouldFilter = false;
	FString FullName = InPackageInfo.PackageName.ToString();
	if (IsTakeSyncEnabled() && WeakSession.IsValid() && !CanRecord() && bSkipPackageFilter == 0)
	{
		FTakeRecorderProjectParameters Project = GetDefault<UTakeRecorderProjectSettings>()->Settings;
		if (  (  FullName.Contains(Project.RootTakeSaveDir.Path)
			  || FullName.Contains(Project.GetResolvedRootTakeSaveDir()))
			&& FullName.Contains(UE::TakeRecorderManager::Private::TempPostfix))
		{
			bShouldFilter = true;
		}
	}
	FString YesOrNo = bShouldFilter ? TEXT("Yes") : TEXT("UseDefault");
	UE_LOGF(LogConcertTakeRecorder, Verbose, "Considering package %ls for filtering: %ls", *FullName, *YesOrNo);
	return bShouldFilter ? EPackageFilterResult::Exclude : EPackageFilterResult::UseDefault;
}

bool FConcertTakeRecorderManager::CanSkipHotReload(const FConcertPackageInfo& InPackageInfo)
{
	if (IsTakeSyncEnabled() && WeakSession.IsValid() && CanRecord() && bEnableSkipHotReloadHint > 0)
	{
		FTakeRecorderProjectParameters Project = GetDefault<UTakeRecorderProjectSettings>()->Settings;
		FString FullName = InPackageInfo.PackageName.ToString();
		return FullName.Contains(Project.RootTakeSaveDir.Path);
	}
	return false;
}

ETransactionFilterResult FConcertTakeRecorderManager::ShouldObjectBeTransacted(const FConcertTransactionFilterArgs& FilterArgs)
{
	UConcertSessionRecordSettings const* RecordSettings = GetDefault<UConcertSessionRecordSettings>();

	UTakePreset* TakePreset = Preset;
	if (WeakSession.IsValid()
		&& TakePreset
		&& RecordSettings->LocalSettings.bTransactSources
		&& FilterArgs.Package
		&& TakePreset->GetOutermost()->GetFName() == FilterArgs.Package->GetFName())
	{
		return ETransactionFilterResult::IncludeObject;
	}

	UConcertTakeSynchronization const* TakeSync	= GetDefault<UConcertTakeSynchronization>();
	if (WeakSession.IsValid()
		&& TakeSync->bTransactTakeMetadata
		&& FilterArgs.ObjectToFilter
		&& FilterArgs.ObjectToFilter->IsA<UTakeMetaData>())
	{
		return ETransactionFilterResult::IncludeObject;
	}

	return ETransactionFilterResult::UseDefault;
}

SIZE_T FConcertTakeRecorderManager::RemoteRecorders() const
{
	UConcertSessionRecordSettings const* RecordSettings = GetDefault<UConcertSessionRecordSettings>();
	SIZE_T Num = Algo::CountIf(RecordSettings->RemoteSettings,
							   [](const FConcertClientRecordSetting&  Remote)
							   {
								   return Remote.Settings.bRecordOnClient;
							   });
	return Num;
}

FTakeRecorderParameters FConcertTakeRecorderManager::SetupTakeParametersForMultiuser(const FTakeRecorderParameters& Input)
{
	if (IsTakeSyncEnabled() && WeakSession.IsValid())
	{
		if (CanRecord() && RemoteRecorders() > 0)
		{
			TSharedPtr<IConcertClientSession> Session = WeakSession.Pin();
			FString LocalName = Session->GetLocalClientInfo().DisplayName;
			if (UE::TakeRecorderManager::Private::IsDuplicateRemoteName(LocalName))
			{
				static FString RandomSessionId =
					FGuid::NewGuid().ToString(EGuidFormats::Short);
				LocalName = LocalName + "_" + RandomSessionId;
			}

			FString Name = UPackageTools::SanitizePackageName(LocalName);
			Name.RemoveSpacesInline();
			FTakeRecorderParameters Output = Input;
			Output.Project.TakeSaveDir = Input.Project.TakeSaveDir + "_" + Name;
			return Output;
		}
		else if (!CanRecord())
		{
			FTakeRecorderParameters Output = Input;
			Output.Project.TakeSaveDir = Input.Project.TakeSaveDir + UE::TakeRecorderManager::Private::TempPostfix;
			Output.User.bSaveRecordedAssets = false;
			return Output;
		}
	}

	return Input;
}

void FConcertTakeRecorderManager::OnNamedLevelSequenceEvent(const FConcertSessionContext&, const FConcertRecordingNamedLevelSequenceEvent& InEvent)
{
	LastLevelSequencePath = InEvent.LevelSequencePath;
}

bool FConcertTakeRecorderManager::CanReviewLastRecordedSequence() const
{
	if (WeakSession.IsValid())
	{
		// Due to when packages arrive in Multi-user, we may have received the name event message before the package is actually saved locally. To
		// prevent loading issues, we need to check to see if that package has arrived and then load it to be able to review the last recording. This
		// method is called by the panel to refresh the button used to review the last take but it requires a const call so LastLevelSequencePath and
		// LastLevelSequence are mutable to allow changing in the const method.
		if (!LastLevelSequencePath.IsEmpty())
		{
			FString PackageName;
			FString AssetName;
			LastLevelSequencePath.Split(TEXT("."), &PackageName, &AssetName);
			if (FPackageName::DoesPackageExist(PackageName, nullptr))
			{
				LastLevelSequence = LoadObject<ULevelSequence>(nullptr, *LastLevelSequencePath);
				
				ITakeRecorderModule& Module = FTakeRecorderRecorderManagerGetModule();
				Module.GetLastLevelSequenceProvider().ExecuteIfBound(LastLevelSequence);

				LastLevelSequencePath.Reset();
			}
		}
		return LastLevelSequence != nullptr;
	}
	return true;
}

#undef LOCTEXT_NAMESPACE /*ConcertTakeRecorder*/
