// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Optional.h"
#include "Templates/FunctionFwd.h"
#include "UObject/GCObject.h"

class FConcertTakeRecorderClientSessionCustomization;
class IConcertClientSession;
class IConcertSyncClient;
class ULevelSequence;
class UTakePreset;
class UTakeRecorder;
class SWidget;

enum class ECheckBoxState : uint8;
enum class EConcertClientStatus : uint8;
enum class EConcertConnectionStatus : uint8;
enum class EPackageFilterResult : uint8;
enum class ETransactionFilterResult : uint8;

struct FConcertClientRecordSetting;
struct FConcertMultiUserSyncChangeEvent;
struct FConcertPackageInfo;
struct FConcertRecordSettingsChangeEvent;
struct FConcertRecordingCancelledEvent;
struct FConcertRecordingFinishedEvent;
struct FConcertRecordingNamedLevelSequenceEvent;
struct FConcertSessionContext;
struct FConcertSessionClientInfo;
struct FConcertTakeInitializedEvent;
struct FConcertTransactionFilterArgs;
struct FFrameNumber;
struct FTakeRecorderParameters;
struct FTakeRecordSettings;
struct EVisibility;

DECLARE_LOG_CATEGORY_EXTERN(LogConcertTakeRecorder, Log, All);

/**
 * Take Recorder manager that is held by the client sync module that keeps track of when a take is started, stopped or cancelled.
 * Events are registered to client sessions that will then operate on the Take Recorder UIs
 */
class FConcertTakeRecorderManager : public FGCObject
{
public:
	
	/** Registers TakeRecorderInitialized handler with the take recorder module */
	FConcertTakeRecorderManager();
	/** Unregisters TakeRecorderInitialized handler from the take recorder module */
	virtual ~FConcertTakeRecorderManager() override;

	/**
	 * Register all custom take recorder events for the specified client session
	 *
	 * @param InSession	The client session to register custom events with
	 */
	void Register(TSharedRef<IConcertClientSession> InSession);
	/**
	 * Unregister previously registered custom take recorder events from the specified client session
	 *
	 * @param InSession The client session to unregister custom events from
	 */
	void Unregister(TSharedRef<IConcertClientSession> InSession);

	/** Attempts the client recorder settings for the given client. The other clients inform us when they are changed so they are somewhat in sync. Nullptr if the client is unknown. */
	const FConcertClientRecordSetting* FindClientRecorderSetting(const FGuid& EndpointId) const;
	/**
	 * Modifies the settings of a client, if the client exists. Subsequently updates all the relevant systems of the change.
	 * @return Whether the client was found.
	 */
	bool EditClientSettings(
		const FGuid& EndpointId,
		TFunctionRef<void(FTakeRecordSettings& Settings)> ModifierFunc,
		TOptional<TFunctionRef<bool(const FTakeRecordSettings& Settings)>> Predicate = {}
		);

private:
	//~ Take recorder delegate handlers
	void OnTakeRecorderInitialized(UTakeRecorder* TakeRecorder);
	void OnRecordingFinished(UTakeRecorder* TakeRecorder);
	void OnRecordingCancelled(UTakeRecorder* TakeRecorder);
	void OnTakeRecorderStarted(UTakeRecorder* TakeRecorder);
	void OnTakeRecorderStopped(UTakeRecorder* TakeRecorder);

	void OnFrameAdjustment(UTakeRecorder* TakeRecorder, const FFrameNumber& InPlaybackStartFrame);

	//~ Concert event handlers
	void OnTakeInitializedEvent(const FConcertSessionContext&, const FConcertTakeInitializedEvent& InEvent);
	void OnRecordingFinishedEvent(const FConcertSessionContext&, const FConcertRecordingFinishedEvent&);
	void OnRecordingCancelledEvent(const FConcertSessionContext&, const FConcertRecordingCancelledEvent&);

	void OnRecordSettingsChangeEvent(const FConcertSessionContext&, const FConcertRecordSettingsChangeEvent&);
	void OnMultiUserSyncChangeEvent(const FConcertSessionContext&, const FConcertMultiUserSyncChangeEvent&);

	void OnNamedLevelSequenceEvent(const FConcertSessionContext&, const FConcertRecordingNamedLevelSequenceEvent&);

	void OnSessionClientChanged(IConcertClientSession&, EConcertClientStatus ClientStatus, const FConcertSessionClientInfo& InClientInfo);
	void OnSessionConnectionChanged(IConcertClientSession&, EConcertConnectionStatus ConnectionStatus);

	//~ Widget extension handlers
	void RegisterExtensions();
	void UnregisterExtensions();

	void CreateExtensionWidget(TArray<TSharedRef<class SWidget>>& OutExtensions);
	void CreateRecordButtonOverlay(TArray<TSharedRef<SWidget>>& OutExtensions);

	SIZE_T RemoteRecorders() const;
	bool ShouldIconBeVisible() const;
	EVisibility GetMultiUserIconVisibility() const;

	bool IsTakeSyncEnabled() const;
	ECheckBoxState IsTakeSyncChecked() const;
	void HandleTakeSyncCheckBox(ECheckBoxState State) const;
	EVisibility HandleTakeSyncButtonVisibility() const;

	void UpdateSessionClientList();
	void DisconnectFromSession();
	void ConnectToSession(IConcertClientSession&);

	ETransactionFilterResult ShouldObjectBeTransacted(const FConcertTransactionFilterArgs& FilterArgs);
	EPackageFilterResult ShouldPackageBeFiltered(const FConcertPackageInfo& InPackage);
	bool CanSkipHotReload(const FConcertPackageInfo& InPackage);
private:
	FTakeRecorderParameters SetupTakeParametersForMultiuser(const FTakeRecorderParameters& Input);

	virtual void AddReferencedObjects( FReferenceCollector& Collector ) override
	{
		Collector.AddReferencedObject(Preset);
		Collector.AddReferencedObject(LastLevelSequence);
	}

	virtual FString GetReferencerName() const override
	{
		return TEXT("FConcertTakeRecorderManager");
	}

	void ReportRecordingError(FText &);
	bool CanRecord() const;
	bool CanAnyRecord() const;

	void SendInitialState(IConcertClientSession&);
	void OnTakeSyncPropertyChange(bool Value);

	void RecordSettingChange(const FConcertClientRecordSetting& RecordSetting);
	void AddRemoteClient(const FConcertSessionClientInfo& ClientInfo);

	bool CanReviewLastRecordedSequence() const;
	
	/** Weak pointer to the client session with which to send events. May be null or stale. */
	TWeakPtr<IConcertClientSession> WeakSession;

	/**
	 * Used to prevent sending out events that were just received by this client.
	 */
	struct FTakeRecorderState
	{
		FString LastStartedTake;
		FString LastStoppedTake;
	} TakeRecorderState;

	TObjectPtr<UTakePreset> Preset = nullptr;

	mutable TObjectPtr<ULevelSequence> LastLevelSequence = nullptr;
	mutable FString LastLevelSequencePath;
	
	TSharedPtr<FConcertTakeRecorderClientSessionCustomization> Customization;

	/** Delegate for any changes in client state. */
	FDelegateHandle				 ClientChangeDelegate;

	/** Delegate for any changes take sync status. */
	FDelegateHandle				 TakeSyncDelegate;

	/**
	 * Denotes if we are currently in recording mode.
	 */
	bool bIsRecording = false;

	/** Indicates if we have warned the user about multiple recorders. */
	bool bHaveWarned = false;
};
