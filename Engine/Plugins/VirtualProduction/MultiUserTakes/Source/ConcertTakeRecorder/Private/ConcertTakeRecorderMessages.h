// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Recorder/TakeRecorderParameters.h"
#include "IdentifierTable/ConcertIdentifierTableData.h"

#include "ConcertMessageData.h"
#include "ConcertTakeRecorderMessages.generated.h"

UCLASS(config=Engine, DisplayName="Multi-user Take Synchronization")
class UConcertTakeSynchronization : public UObject
{
public:
	GENERATED_BODY()

	UPROPERTY(config,EditAnywhere,BlueprintReadWrite,Category="Multi-user Take Synchronization",DisplayName="Synchronize Take Recorder Transactions")
	bool bSyncTakeRecordingTransactions = true;

	UPROPERTY(config,EditAnywhere,BlueprintReadWrite,Category="Multi-user Take Synchronization")
	bool bTransactTakeMetadata = true;
};

USTRUCT()
struct FTakeRecordSettings
{
	GENERATED_BODY();

	UPROPERTY(config,EditAnywhere,Category="Multi-user Client Record Settings")
	bool bRecordOnClient = false;

	UPROPERTY(config,EditAnywhere,Category="Multi-user Client Record Settings")
	bool bTransactSources = true;
};

USTRUCT()
struct FConcertClientRecordSetting
{
	GENERATED_BODY()

	UPROPERTY()
	FConcertSessionClientInfo Details;

	UPROPERTY()
	bool bTakeSyncEnabled = true;

	UPROPERTY()
	FTakeRecordSettings Settings;
};

UCLASS(config=Engine, DisplayName="Multi-user Client Record Settings")
class UConcertSessionRecordSettings : public UObject
{
public:
	GENERATED_BODY()

	UPROPERTY(config,EditAnywhere,Category="Multi-user Client Record Settings")
	FTakeRecordSettings LocalSettings;

	UPROPERTY(Transient,EditAnywhere,Category="Multi-user Client Record Settings")
	TArray<FConcertClientRecordSetting> RemoteSettings;
};

USTRUCT()
struct FConcertRecordSettingsChangeEvent
{
	GENERATED_BODY()

	UPROPERTY()
	FGuid EndpointId;

	UPROPERTY()
	FTakeRecordSettings Settings;
};

USTRUCT()
struct FConcertMultiUserSyncChangeEvent
{
	GENERATED_BODY()

	UPROPERTY()
	FGuid EndpointId;

	UPROPERTY()
	bool bSyncTakeRecordingTransactions = true;
};


USTRUCT()
struct FConcertTakeInitializedEvent
{
	GENERATED_BODY()

	UPROPERTY()
	FString TakePresetPath;

	UPROPERTY()
	FString TakeName;

	UPROPERTY()
	TArray<uint8> TakeData;

	UPROPERTY()
	FString TakeMetaDataPath;

	UPROPERTY()
	FConcertLocalIdentifierState IdentifierState;

	UPROPERTY()
	FTakeRecorderUserParameters Settings;

	UPROPERTY()
	FTakeRecorderProjectParameters Project;
};

USTRUCT()
struct FConcertRecordingFinishedEvent
{
	GENERATED_BODY()

	UPROPERTY()
	FString TakeName;
};

USTRUCT()
struct FConcertRecordingNamedLevelSequenceEvent
{
	GENERATED_BODY()

	UPROPERTY()
	FString LevelSequencePath;
};

USTRUCT()
struct FConcertRecordingCancelledEvent
{
	GENERATED_BODY()

	UPROPERTY()
	FString TakeName;
};
