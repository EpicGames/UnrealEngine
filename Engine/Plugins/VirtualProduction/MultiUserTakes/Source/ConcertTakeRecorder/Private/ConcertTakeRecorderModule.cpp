// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertTakeRecorderModule.h"

#include "ConcertTakeRecorderManager.h"
#include "ConcertTakeRecorderStyle.h"

LLM_DEFINE_TAG(Concert_ConcertTakeRecorder);

namespace UE::ConcertTakeRecorder
{
	void FConcertTakeRecorderModule::StartupModule()
	{
		LLM_SCOPE_BYTAG(Concert_ConcertTakeRecorder);
		
		FConcertTakeRecorderStyle::Initialize();
		ConcertManager = MakeUnique<FConcertTakeRecorderManager>();
	}
	
	void FConcertTakeRecorderModule::ShutdownModule()
	{
		ConcertManager.Reset();
		FConcertTakeRecorderStyle::Shutdown();
	}
}

IMPLEMENT_MODULE(UE::ConcertTakeRecorder::FConcertTakeRecorderModule, ConcertTakeRecorder);
