// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConcertTakeRecorderManager.h"
#include "Modules/ModuleManager.h"
#include "Templates/UniquePtr.h"

namespace UE::ConcertTakeRecorder
{
	/** Module that adds multi user synchronization to take recorder. */
	class FConcertTakeRecorderModule : public IModuleInterface
	{
	public:

		/**
		 * Singleton-like access to this module's interface.  This is just for convenience!
		 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
		 *
		 * @return Returns singleton instance, loading the module on demand if needed
		 */
		static inline FConcertTakeRecorderModule& Get()
		{
			static const FName ModuleName = "ConcertTakeRecorder";
			return FModuleManager::LoadModuleChecked<FConcertTakeRecorderModule>(ModuleName);
		}
		
		//~ Begin IModuleInterface Interface
		virtual void StartupModule() override;
		virtual void ShutdownModule() override;
		//~ End IModuleInterface Interface

		FConcertTakeRecorderManager* GetTakeRecorderManager() const { return ConcertManager.Get(); }
		
	private:

		/** Synchronizes takes across clients. Valid for the lifetime of the module. */
		TUniquePtr<FConcertTakeRecorderManager> ConcertManager;
	};
}
