// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertTakeRecorderStyle.h"
#include "Brushes/SlateImageBrush.h"
#include "Styling/SlateStyleRegistry.h"
#include "Interfaces/IPluginManager.h"
#include "Misc/Paths.h"
#include "Styling/SlateStyle.h"

#define IMAGE_PLUGIN_BRUSH( RelativePath, ... ) FSlateImageBrush( FConcertTakeRecorderStyle::InContent( RelativePath, ".png" ), __VA_ARGS__ )
#define IMAGE_BRUSH(RelativePath, ...) FSlateImageBrush(StyleSet->RootToContentDir(RelativePath, TEXT(".png")), __VA_ARGS__)

FString FConcertTakeRecorderStyle::InContent(const FString& RelativePath, const ANSICHAR* Extension)
{
	static FString ContentDir = IPluginManager::Get().FindPlugin(TEXT("MultiUserTakes"))->GetContentDir();
	return (ContentDir / RelativePath) + Extension;
}

TSharedPtr< class FSlateStyleSet > FConcertTakeRecorderStyle::StyleSet;

FName FConcertTakeRecorderStyle::GetStyleSetName()
{
	return FName(TEXT("ConcertTakeRecorderStyle"));
}

void FConcertTakeRecorderStyle::Initialize()
{
	// Only register once
	if (StyleSet.IsValid())
	{
		return;
	}

	StyleSet = MakeShared<FSlateStyleSet>(GetStyleSetName());
	StyleSet->SetContentRoot(FPaths::EngineContentDir() / TEXT("Editor/Slate"));
	StyleSet->SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

	// Const icon sizes
	const FVector2D Icon16x16(16.0f, 16.0f);
	const FVector2D Icon24x24(24.0f, 24.0f);
	const FVector2D Icon48x48(48.0f, 48.0f);

	// Most icons were designed to be used at 80% opacity.
	const FLinearColor IconColorAndOpacity(FLinearColor(1.f, 1.f, 1.f, 0.8f));

	// 24x24/48x48 -> For take recorder toolbar.
	StyleSet->Set("Concert.TakeRecorder.SyncTakes",       new IMAGE_PLUGIN_BRUSH("icon_TakeRecorderSync_48x", Icon48x48, IconColorAndOpacity)); // Enable/disable take syncing on a remote client.
	StyleSet->Set("Concert.TakeRecorder.SyncTakes.Small", new IMAGE_PLUGIN_BRUSH("icon_TakeRecorderSync_48x", Icon24x24, IconColorAndOpacity));
	StyleSet->Set("Concert.TakeRecorder.SyncTakes.Tiny", new IMAGE_PLUGIN_BRUSH("icon_TakeRecorderSync_48x", Icon16x16, IconColorAndOpacity));
	FSlateStyleRegistry::RegisterSlateStyle(*StyleSet.Get());
};

void FConcertTakeRecorderStyle::Shutdown()
{
	if (StyleSet.IsValid())
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*StyleSet.Get());
		ensure(StyleSet.IsUnique());
		StyleSet.Reset();
	}
}

TSharedPtr<class ISlateStyle> FConcertTakeRecorderStyle::Get()
{
	return StyleSet;
}

#undef IMAGE_PLUGIN_BRUSH
#undef IMAGE_BRUSH

