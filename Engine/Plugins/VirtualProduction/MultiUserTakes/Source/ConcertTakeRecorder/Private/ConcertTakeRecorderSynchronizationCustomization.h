// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "DetailLayoutBuilder.h"
#include "ConcertTakeRecorderMessages.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Input/SHyperlink.h"
#include "DetailWidgetRow.h"

#include "Modules/ModuleManager.h"
#include "ISettingsModule.h"

class IModuleInterface;

#define LOCTEXT_NAMESPACE "ConcertTakeRecorderCustomization"

DECLARE_MULTICAST_DELEGATE_OneParam(FOnSyncPropertyValueChanged, bool);

class FConcertTakeRecorderSynchronizationCustomization : public IDetailCustomization
{
public:
	using ThisType = FConcertTakeRecorderSynchronizationCustomization;
	static void OpenMultiUserExcludeProperty()
	{
		FModuleManager::GetModulePtr<ISettingsModule>("Settings")->ShowViewer("Project", "Plugins", "Concert Sync");
	}

	static FOnSyncPropertyValueChanged& OnSyncPropertyValueChanged()
	{
		static FOnSyncPropertyValueChanged SyncPropertyChanged;
		return SyncPropertyChanged;
	}

	void TakeMetaPropertyChanged()
	{
		UConcertTakeSynchronization* TakeSync = GetMutableDefault<UConcertTakeSynchronization>();
		TakeSync->SaveConfig();
	}

	void SyncPropertyChanged()
	{
		UConcertTakeSynchronization const* TakeSync = GetDefault<UConcertTakeSynchronization>();
		OnSyncPropertyValueChanged().Broadcast(TakeSync->bSyncTakeRecordingTransactions);
	}

	void CustomizeDetails(IDetailLayoutBuilder& DetailLayout) override
	{
		// Pop the take recorder category to the top
		IDetailCategoryBuilder& TakeSync = DetailLayout.EditCategory("Multi-user Take Synchronization");

		TSharedPtr<IPropertyHandle> TransactMetaData = DetailLayout.GetProperty(
			GET_MEMBER_NAME_CHECKED(UConcertTakeSynchronization, bTransactTakeMetadata));
		TransactMetaData->SetOnChildPropertyValueChanged(FSimpleDelegate::CreateRaw(this, &ThisType::TakeMetaPropertyChanged));

		TSharedPtr<IPropertyHandle> SyncTakeRecordingProperty = DetailLayout.GetProperty(
			GET_MEMBER_NAME_CHECKED(UConcertTakeSynchronization, bSyncTakeRecordingTransactions));
		SyncTakeRecordingProperty->MarkHiddenByCustomization();
		SyncTakeRecordingProperty->SetOnPropertyValueChanged(FSimpleDelegate::CreateRaw(this, &ThisType::SyncPropertyChanged));
		{
			TSharedPtr<SWidget> NameWidget;
			TSharedPtr<SWidget> ValueWidget;

			IDetailPropertyRow& SyncDisplayProperty = TakeSync.AddProperty(SyncTakeRecordingProperty);
			SyncDisplayProperty.GetDefaultWidgets(NameWidget,ValueWidget);
			FDetailWidgetRow& Row = SyncDisplayProperty.CustomWidget(true);

			Row.NameContent()
			[
				NameWidget.ToSharedRef()
			];

			Row.ValueContent()
			.MinDesiredWidth(250)
			.MaxDesiredWidth(0)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.FillWidth(1.0)
				.HAlign(HAlign_Left)
				[
					SNew(SBox)
					.WidthOverride(250.f)
					[
						ValueWidget.ToSharedRef()
					]
				]
				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Center)
				.AutoWidth()
				[
					SNew(SHyperlink)
					.Style(FAppStyle::Get(), "Common.GotoBlueprintHyperlink")
					.OnNavigate(FSimpleDelegate::CreateLambda([](){OpenMultiUserExcludeProperty();}))
					.Text((LOCTEXT("SetExclusionFilter","Set Exclusion Filters")))
					.ToolTipText(LOCTEXT("SetExclusionFilterToolTip","Click to edit transaction exclusion filter for MultiUser"))
				]
			];
		}
	}
};

#undef LOCTEXT_NAMESPACE
