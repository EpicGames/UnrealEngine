// Copyright Epic Games, Inc. All Rights Reserved.

#include "MultiUserTakesFunctionLibrary.h"

#include "ConcertLogGlobal.h"
#include "ConcertTakeRecorderManager.h"
#include "ConcertTakeRecorderMessages.h"
#include "ConcertTakeRecorderModule.h"
#include "ConcertTakeRecorderSynchronizationCustomization.h"
#include "IConcertSyncClient.h"
#include "IConcertSyncClientModule.h"

#include "Templates/Function.h"

bool UMultiUserTakesFunctionLibrary::GetRecordOnClientLocal()
{
	const TSharedPtr<IConcertSyncClient> ConcertSyncClient = IConcertSyncClientModule::Get().GetClient(TEXT("MultiUser"));
	const TSharedPtr<IConcertClientSession> Session = ConcertSyncClient ? ConcertSyncClient->GetConcertClient()->GetCurrentSession() : nullptr;
	
	UE_CLOGF(!Session, LogConcert, Warning, "MultiUserTakes: Cannot change local RecordOnClient state because this editor is nott in any session!");
	return Session && GetRecordOnClient(Session->GetSessionClientEndpointId());
}

void UMultiUserTakesFunctionLibrary::SetRecordOnClientLocal(bool bNewValue)
{
	const TSharedPtr<IConcertSyncClient> ConcertSyncClient = IConcertSyncClientModule::Get().GetClient(TEXT("MultiUser"));
	const TSharedPtr<IConcertClientSession> Session = ConcertSyncClient ? ConcertSyncClient->GetConcertClient()->GetCurrentSession() : nullptr;
	if (Session)
	{
		SetRecordOnClient(Session->GetSessionClientEndpointId(), bNewValue);
	}
	else
	{
		UE_LOGF(LogConcert, Warning, "MultiUserTakes: Cannot change local RecordOnClient state because this editor is nott in any session!");
	}
}

bool UMultiUserTakesFunctionLibrary::GetRecordOnClient(const FGuid& ClientEndpointId)
{
	using namespace UE::ConcertTakeRecorder;
	
	if (const FConcertTakeRecorderManager* Manager = FConcertTakeRecorderModule::Get().GetTakeRecorderManager())
	{
		const FConcertClientRecordSetting* Settings = Manager->FindClientRecorderSetting(ClientEndpointId);
		return Settings && Settings->Settings.bRecordOnClient;
	}
	return false;
}

void UMultiUserTakesFunctionLibrary::SetRecordOnClient(const FGuid& ClientEndpointId, bool bNewValue)
{
	using namespace UE::ConcertTakeRecorder;
	
	if (FConcertTakeRecorderManager* Manager = FConcertTakeRecorderModule::Get().GetTakeRecorderManager())
	{
		const bool bClientWasFound = Manager->EditClientSettings(
			ClientEndpointId,
			[bNewValue](FTakeRecordSettings& Settings) { Settings.bRecordOnClient = bNewValue; },
			{ [bNewValue](const FTakeRecordSettings& Settings){ return Settings.bRecordOnClient != bNewValue; } }
			);
	}
}

bool UMultiUserTakesFunctionLibrary::GetSynchronizeTakeRecorderTransactionsLocal()
{
	return GetMutableDefault<UConcertTakeSynchronization>()->bSyncTakeRecordingTransactions;
}

bool UMultiUserTakesFunctionLibrary::GetSynchronizeTakeRecorderTransactions(const FGuid& ClientEndpointId)
{
	using namespace UE::ConcertTakeRecorder;
	
	if (const FConcertTakeRecorderManager* Manager = FConcertTakeRecorderModule::Get().GetTakeRecorderManager())
	{
		const FConcertClientRecordSetting* Settings = Manager->FindClientRecorderSetting(ClientEndpointId);
		return Settings && Settings->bTakeSyncEnabled;
	}
	return false;
}

void UMultiUserTakesFunctionLibrary::SetSynchronizeTakeRecorderTransactionsLocal(bool bNewValue)
{
	UConcertTakeSynchronization* Settings = GetMutableDefault<UConcertTakeSynchronization>();
	if (Settings->bSyncTakeRecordingTransactions != bNewValue)
	{
		Settings->bSyncTakeRecordingTransactions = bNewValue;

		// Causes all the other clients to be notified. This should REALLY live in UConcertTakeSynchronization.
		FConcertTakeRecorderSynchronizationCustomization::OnSyncPropertyValueChanged().Broadcast(bNewValue);
	}
}
