// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"
#include "MultiUserTakesFunctionLibrary.generated.h"

/** Exposes some of the recorder settings to Blueprints to allow for more editor scripting capabilities. */
UCLASS()
class CONCERTTAKERECORDER_API UMultiUserTakesFunctionLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:

	/**
	 * Gets the checkbox value in the "Record On Client" column in the settings displayed at the bottom of the "Take Recorder" tab.
	 * This is an utility for getting the local client's value; GetRecordOnClient can also be used.
	 */
	UFUNCTION(BlueprintPure, Category = "Multi User Takes")
	static bool GetRecordOnClientLocal();

	/**
	 * Sets the checkbox value in the "Record On Client" column in the settings displayed at the bottom of the "Take Recorder" tab.
	 * This is an utility for setting the local client's value; SetRecordOnClient can also be used.
	 */
	UFUNCTION(BlueprintCallable, Category = "Multi User Takes")
	static void SetRecordOnClientLocal(bool bNewValue);

	/** 
	 * Gets the checkbox value in the "Record On Client" column in the settings displayed at the bottom of the "Take Recorder" tab.
	 * This function queries the state of any connected client. For the local client, you can also use GetRecordOnClientLocal.
	 *
	 * @param ClientEndpointId ID of the client. You can get this by using e.g.UMultiUserSubsystem::GetRemoteClientIds or UMultiUserSubsystem::GetLocalClientId.
	 */
	UFUNCTION(BlueprintPure, Category = "Multi User Takes")
	static bool GetRecordOnClient(const FGuid& ClientEndpointId);
	
	/** 
	 * Sets the checkbox value in the "Record On Client" column in the settings displayed at the bottom of the "Take Recorder" tab.
	 * This function queries the state of any connected client. For the local client, you can also use SetRecordOnClientLocal.
	 *
	 * @param ClientEndpointId ID of the client. You can get this by using e.g.UMultiUserSubsystem::GetRemoteClientIds or UMultiUserSubsystem::GetLocalClientId.
	 * @param bNewValue The new value to set
	 */
	UFUNCTION(BlueprintCallable, Category = "Multi User Takes")
	static void SetRecordOnClient(const FGuid& ClientEndpointId, bool bNewValue);

	/**
	 * Gets the value of the "SynchronizeTakeRecorderTransactions" checkbox in the settings displayed at the bottom of the "Take Recorder" tab.
	 * This is an utility for getting the local client's value; GetSynchronizeTakeRecorderTransactions can also be used.
	 */
	UFUNCTION(BlueprintPure, Category = "Multi User Takes")
	static bool GetSynchronizeTakeRecorderTransactionsLocal();

	/**
	 * Gets the value of the "SynchronizeTakeRecorderTransactions" checkbox in the settings displayed at the bottom of the "Take Recorder" tab.
	 * This function queries the state of any connected client. For the local client, you can also use GetSynchronizeTakeRecorderTransactionsLocal.
	 *
	 * @param ClientEndpointId ID of the client. You can get this by using e.g.UMultiUserSubsystem::GetRemoteClientIds or UMultiUserSubsystem::GetLocalClientId.
	 */
	UFUNCTION(BlueprintPure, Category = "Multi User Takes")
	static bool GetSynchronizeTakeRecorderTransactions(const FGuid& ClientEndpointId);

	/**
	 * Sets the value of the "SynchronizeTakeRecorderTransactions" checkbox in the settings displayed at the bottom of the "Take Recorder" tab.
	 * @note Only the value of the local client can be set. Setting of remote clients is not implemented (no technical reason - there just never was a use-case). 
	 */
	UFUNCTION(BlueprintCallable, Category = "Multi User Takes")
	static void SetSynchronizeTakeRecorderTransactionsLocal(bool bNewValue);
};
